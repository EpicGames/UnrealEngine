{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "1.1",
	"FriendlyName" : "Datasmith C4D Importer",
	"Description" : "Adds support for importing content from Cinema4D applications into Unreal Engine",
	"Category" : "Importers",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/en-US/WorkingWithContent/Importing/Datasmith/",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedTargetPlatforms":
	[
		"Win64",
		"Mac"
	],
	"Modules" :
	[
		{
			"Name" : "DatasmithC4DTranslator",
			"Type" : "Runtime",
			"LoadingPhase" : "PostDefault",
			"PlatformAllowList" :
			[
				"Win64",
				"Mac"
			]
		}		
	],
	"Plugins":
	[
		{
			"Name": "DatasmithImporter",
			"Enabled": true
		}
	]
}
