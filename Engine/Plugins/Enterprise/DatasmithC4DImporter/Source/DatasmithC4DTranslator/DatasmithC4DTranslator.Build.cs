// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithC4DTranslator : ModuleRules
	{
		public DatasmithC4DTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			bLegalToDistributeObjectCode = true;
			bUseUnity = false;

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"Core",
					"CoreUObject",
					"DatasmithCore",
					"Engine",
					"MeshDescription",
					"StaticMeshDescription",
					"Imath",
				}
			);

			if (Target.Type == TargetType.Editor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"DatasmithExporter",
						"Slate",
						"SlateCore",
						"EditorStyle",
					}
				);
			}

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithContent",
					"DatasmithTranslator"
				}
			);


			// no winarm64 sdk exists yet
			if (!(Target.Platform == UnrealTargetPlatform.Win64 && Target.Architecture == UnrealArch.Arm64))
			{
				// Set up the C4D Melange SDK includes and libraries.
				string MelangeSDKLocation = Path.Combine(EngineDirectory, "Restricted/NotForLicensees/Source/ThirdParty/Enterprise/Melange/20.004_RBMelange20.0_259890");

				// When C4D Melange SDK is not part of the developer's workspace, look for environment variable Melange_SDK.
				if (!Directory.Exists(MelangeSDKLocation))
				{
					MelangeSDKLocation = System.Environment.GetEnvironmentVariable("Melange_SDK");
				}

				// Make sure the C4D Melange SDK is actually installed.
				if (Directory.Exists(MelangeSDKLocation))
				{
					PublicDefinitions.Add("_MELANGE_SDK_");
					PrivateDependencyModuleNames.Add("MelangeSDK");
				}
			}

			if (Target.WindowsPlatform.Compiler == WindowsCompiler.Clang)
			{
				PublicDefinitions.Add("WITH_CLANG_COMPILER");
			}
		}
	}
}
