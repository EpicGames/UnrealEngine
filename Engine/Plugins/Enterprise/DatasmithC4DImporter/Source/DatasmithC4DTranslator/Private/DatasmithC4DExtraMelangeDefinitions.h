// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// For some reason some symbols are not actually defined within cineware, and are
// passed around just as raw int32 ids.
// Here we either define or inject them into the cineware namespace like the others

#define Ocloner 1018544
#define Ofracture 1018791

#define REFLECTION_LAYER_MAIN_OPACITY REFLECTION_LAYER_TRANS_BRIGHTNESS
#define REFLECTION_LAYER_NAME 0x0015

namespace cineware
{
	// We use C-style enums here instead of defines so that they are contained
	// within the cineware namespace and behave like the other enums

    enum // Tcrane attributes
    {
		//CRANECAMERA_UNKNOWN_LINK = 898,
		CRANECAMERA_NAME = 900,
		//CRANECAMERA_UNKNOWN_LONG = 100101, // "Link"
		//CRANECAMERA_UNKNOWN_VEC = 100102,
		//CRANECAMERA_UNKNOWN_VEC = 100103,
		//CRANECAMERA_UNKNOWN_REAL = 100203,
		//CRANECAMERA_UNKNOWN_LONG = 100204,
		CRANECAMERA_BASE_HEIGHT = 100205, //double, cm
		CRANECAMERA_BASE_HEADING = 100206, // double, radians
		CRANECAMERA_ARM_LENGTH = 100211, // double, cm
		CRANECAMERA_ARM_PITCH = 100212, // double, radians
		CRANECAMERA_HEAD_HEIGHT = 100221, // double, cm
		CRANECAMERA_HEAD_HEADING = 100222, // double, radians
		CRANECAMERA_HEAD_WIDTH = 100223, // double, cm
		CRANECAMERA_CAM_PITCH = 100231, // double, radians
		CRANECAMERA_CAM_BANKING = 100232, // double, radians
		CRANECAMERA_CAM_OFFSET = 100233, // double, cm
		CRANECAMERA_COMPENSATE_PITCH = 100234, // bool
		CRANECAMERA_COMPENSATE_HEADING = 100235, // bool
    };

	enum // Tpolygonselection
	{
		POLYGONSELECTIONTAG_NAME = 900,
	};

	enum // object properties
	{
		ID_BASEOBJECT_REL_SIZE = 1100,
	};

	enum // Ocloner (defined in resources folder)
	{
		MGCLONER_VOLUMEINSTANCES_MODE = 1025,
	};
}
