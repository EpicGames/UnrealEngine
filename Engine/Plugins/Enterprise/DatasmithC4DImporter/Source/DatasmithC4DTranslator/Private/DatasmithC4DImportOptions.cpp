// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithC4DImportOptions.h"

#include "DatasmithC4DTranslatorModule.h"

#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "DatasmithC4DImportPlugin"

UDatasmithC4DImportOptions::UDatasmithC4DImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bImportEmptyMesh = false;
	bOptimizeEmptySingleChildActors = false;
	bAlwaysGenerateNormals = false;
	ScaleVertices = 1.0;
#if WITH_EDITOR
	bExportToUDatasmith = false;
#endif //WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
