// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportOptions.h"
#include "DatasmithAssetImportData.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "DatasmithC4DImportOptions.generated.h"

//TODO (C4D): The available options will have to be reviewed since some were specific to the original plug-in.

/*UENUM()
enum class EC4DUVGeneration : uint8
{
	Keep,
	Missing,
	Always,
};*/

UCLASS(config = EditorPerProjectUserSettings, HideCategories = (DebugProperty))
class UDatasmithC4DImportOptions : public UDatasmithOptionsBase
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Import Mesh With No Vertex, false by default
	 */
	UPROPERTY(config, EditAnywhere, Category = DebugProperty, meta = (DisplayName = "Import Mesh With No Vertex"))
	bool bImportEmptyMesh;

	/**
	 * Remove empty actors that have only one child, false by default
	 */
	UPROPERTY(config, EditAnywhere, Category = DebugProperty, meta = (DisplayName = "Clean the Scene of Empty Actors With Only One Child"))
	bool bOptimizeEmptySingleChildActors;

	/**
	* Ignore the normals provided by Melange and let Datasmith generate them
	* The "if (RawNormal.SizeSquared() < SMALL_NUMBER)" check in see FDatasmithMeshUtils::ToMeshDescription is preventing some meshes to load
	*/
	UPROPERTY(config, EditAnywhere, Category = DebugProperty, meta = (DisplayName = "Generate the Normals"))
	bool bAlwaysGenerateNormals;

	/**
	* Scale all position and vertices by this value.
	* It is needed when scene has too precise meshes because the conversion from double (C4D) to float (UnrealEditor)
	* can "degenerate" the faces
	*/
	UPROPERTY(config, EditAnywhere, Category = DebugProperty, meta = (DisplayName = "Scale the Entire Scene"))
	float ScaleVertices;

#if WITH_EDITORONLY_DATA
	/**
	* Export the imported scene as a .datasmith file, next to the .c4d file.
	*/
	UPROPERTY(config, EditAnywhere, Category = DebugProperty, meta = (DisplayName = "Export to .udatasmith"))
	bool bExportToUDatasmith;
#endif //WITH_EDITORONLY_DATA
};
