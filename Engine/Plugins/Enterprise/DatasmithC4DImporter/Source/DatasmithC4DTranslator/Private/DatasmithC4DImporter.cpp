// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MELANGE_SDK_

#include "DatasmithC4DImporter.h"

#include "DatasmithAssetImportData.h"
#include "DatasmithC4DExtraMelangeDefinitions.h"
#include "DatasmithC4DTranslatorModule.h"
#include "DatasmithC4DUtils.h"
#include "DatasmithMesh.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "IDatasmithSceneElements.h"
#include "Utility/DatasmithMeshHelper.h"
#if WITH_EDITOR
#include "DatasmithMeshExporter.h"
#include "DatasmithSceneExporter.h"
#endif //WITH_EDITOR

#include "AssetRegistry/AssetRegistryModule.h"
#include "Components.h"
#include "Curves/RichCurve.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/World.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/Event.h"
#include "HAL/FileManager.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Math/Matrix.h"
#include "Math/Transform.h"
#include "MeshDescription.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "RawMesh.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "Widgets/Notifications/SNotificationList.h"

#include "Imath/ImathMatrixAlgo.h"

DECLARE_CYCLE_STAT(TEXT("C4DImporter - Load File"), STAT_C4DImporter_LoadFile, STATGROUP_C4DImporter);

DEFINE_LOG_CATEGORY_STATIC(LogDatasmithC4DImport, Log, All);

#define LOCTEXT_NAMESPACE "DatasmithC4DImportPlugin"

// What we multiply the C4D light brightness values with when the lights are not
// using photometric units. Those are chosen so that 100% brightness C4D point lights matches the
// default value of 8 candelas of UnrealEditor point lights, and 100% brightness C4D infinite lights matches
// the default 10 lux of UnrealEditor directional lights
#define UnitlessGlobalLightIntensity 10.0
#define UnitlessIESandPointLightIntensity 8000

FDatasmithC4DImporter::FDatasmithC4DImporter(TSharedRef<IDatasmithScene>& OutScene, FDatasmithC4DImportOptions& InOptions)
	: Options(InOptions)
	, DatasmithScene(OutScene)
{
}

FDatasmithC4DImporter::~FDatasmithC4DImporter()
{
	using namespace cineware;

	if (C4dDocument != nullptr)
	{
		DeleteObj(C4dDocument);
		C4dDocument = nullptr;
	}
}

void FDatasmithC4DImporter::SetImportOptions(FDatasmithC4DImportOptions& InOptions)
{
	Options = InOptions;
}

namespace
{
	FMD5Hash ComputePolygonDataHash(cineware::PolygonObject* PolyObject)
	{
		cineware::Int32 PointCount = PolyObject->GetPointCount();
		cineware::Int32 PolygonCount = PolyObject->GetPolygonCount();
		const cineware::Vector* Points = PolyObject->GetPointR();
		const cineware::CPolygon* Polygons = PolyObject->GetPolygonR();
		cineware::Vector32* Normals = PolyObject->CreatePhongNormals();
		cineware::GeData Data;

		FMD5 MD5;
		MD5.Update(reinterpret_cast<const uint8*>(Points), sizeof(cineware::Vector)*PointCount);
		MD5.Update(reinterpret_cast<const uint8*>(Polygons), sizeof(cineware::CPolygon)*PolygonCount);
		if (Normals)
		{
			MD5.Update(reinterpret_cast<const uint8*>(Normals), sizeof(cineware::Vector32)*PointCount);
			cineware::DeleteMem(Normals);
		}

		//Tags
		for (cineware::BaseTag* Tag = PolyObject->GetFirstTag(); Tag; Tag = Tag->GetNext())
		{
			cineware::Int32 TagType = Tag->GetType();
			if (TagType == Tuvw)
			{
				cineware::ConstUVWHandle UVWHandle = static_cast<cineware::UVWTag*>(Tag)->GetDataAddressR();
				for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; ++PolygonIndex)
				{
					cineware::UVWStruct UVWStruct;
					cineware::UVWTag::Get(UVWHandle, PolygonIndex, UVWStruct);
					MD5.Update(reinterpret_cast<const uint8*>(&UVWStruct), sizeof(cineware::UVWStruct));
				}
			}
			else if (TagType == Tpolygonselection)
			{
				cineware::SelectionTag* SelectionTag = static_cast<cineware::SelectionTag*>(Tag);
				cineware::BaseSelect* BaseSelect = SelectionTag->GetBaseSelect();

				FString SelectionName = MelangeGetString(SelectionTag, cineware::POLYGONSELECTIONTAG_NAME);
				uint32 NameHash = GetTypeHash(SelectionName);
				MD5.Update(reinterpret_cast<const uint8*>(&NameHash), sizeof(NameHash));

				TArray<cineware::Int32> PolygonSelections;
				PolygonSelections.Reserve(BaseSelect->GetCount());

				cineware::Int32 Segment = 0;
				cineware::Int32 RangeStart = 0;
				cineware::Int32 RangeEnd = 0;
				cineware::Int32 Selection = 0;
				while (BaseSelect->GetRange(Segment++, &RangeStart, &RangeEnd))
				{
					for (Selection = RangeStart; Selection <= RangeEnd; ++Selection)
					{
						PolygonSelections.Add(Selection);
					}
				}
				MD5.Update(reinterpret_cast<const uint8*>(PolygonSelections.GetData()), PolygonSelections.Num() * sizeof(cineware::Int32));
			}
		}

		FMD5Hash Result;
		Result.Set(MD5);
		return Result;
	}
}

// In C4D the CraneCamera is an object with many attributes that can be manipulated like a
// real-life crane camera. This describes all of its controllable attributes.
// Angles are in degrees, distances in cm. These correspond to the C4D coordinate system
// TODO: Add support Link/Target attributes
struct FCraneCameraAttributes
{
	float BaseHeight = 75.0f;
	float BaseHeading = 0.0f;
	float ArmLength = 300.0f;
	float ArmPitch = 30.0f;
	float HeadHeight = 50.0f;
	float HeadHeading = 0.0f;
	float HeadWidth = 35.0f;
	float CamPitch = 0.0f;
	float CamBanking = 0.0f;
	float CamOffset = 25.0f;
	bool bCompensatePitch = true;
	bool bCompensateHeading = false;

	// Sets one of the attributes using the IDs defined in DatasmithC4DExtraMelangeDefinitions.h
	// Expects the value to be in radians, cm or true/false, depending on attribute
	void SetAttributeByID(int32 AttributeID, double AttributeValue)
	{
		switch (AttributeID)
		{
		case cineware::CRANECAMERA_BASE_HEIGHT:
			BaseHeight = (float)AttributeValue;
			break;
		case cineware::CRANECAMERA_BASE_HEADING:
			BaseHeading = (float)FMath::RadiansToDegrees(AttributeValue);
			break;
		case cineware::CRANECAMERA_ARM_LENGTH:
			ArmLength = (float)AttributeValue;
			break;
		case cineware::CRANECAMERA_ARM_PITCH:
			ArmPitch = (float)FMath::RadiansToDegrees(AttributeValue);
			break;
		case cineware::CRANECAMERA_HEAD_HEIGHT:
			HeadHeight = (float)AttributeValue;
			break;
		case cineware::CRANECAMERA_HEAD_HEADING:
			HeadHeading = (float)FMath::RadiansToDegrees(AttributeValue);
			break;
		case cineware::CRANECAMERA_HEAD_WIDTH:
			HeadWidth = (float)AttributeValue;
			break;
		case cineware::CRANECAMERA_CAM_PITCH:
			CamPitch = (float)FMath::RadiansToDegrees(AttributeValue);
			break;
		case cineware::CRANECAMERA_CAM_BANKING:
			CamBanking = (float)FMath::RadiansToDegrees(AttributeValue);
			break;
		case cineware::CRANECAMERA_CAM_OFFSET:
			CamOffset = (float)AttributeValue;
			break;
		case cineware::CRANECAMERA_COMPENSATE_PITCH:
			bCompensatePitch = static_cast<bool>(AttributeValue);
			break;
		case cineware::CRANECAMERA_COMPENSATE_HEADING:
			bCompensateHeading = static_cast<bool>(AttributeValue);
			break;
		default:
			break;
		}
	}
};

// Extracts all of the relevant parameters from a Tcrane tag and packs them in a FCraneCameraAttributes
TSharedRef<FCraneCameraAttributes> ExtractCraneCameraAttributes(cineware::BaseTag* CraneTag)
{
	TSharedRef<FCraneCameraAttributes> Result = MakeShared<FCraneCameraAttributes>();

	cineware::GeData Data;
	if (CraneTag->GetParameter(cineware::CRANECAMERA_BASE_HEIGHT, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_BASE_HEIGHT, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_BASE_HEADING, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_BASE_HEADING, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_ARM_LENGTH, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_ARM_LENGTH, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_ARM_PITCH, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_ARM_PITCH, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_HEAD_HEIGHT, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_HEAD_HEIGHT, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_HEAD_HEADING, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_HEAD_HEADING, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_HEAD_WIDTH, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_HEAD_WIDTH, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_CAM_PITCH, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_CAM_PITCH, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_CAM_BANKING, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_CAM_BANKING, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_CAM_OFFSET, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_CAM_OFFSET, Data.GetFloat());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_COMPENSATE_PITCH, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_COMPENSATE_PITCH, Data.GetInt32());
	}
	if (CraneTag->GetParameter(cineware::CRANECAMERA_COMPENSATE_HEADING, Data))
	{
		Result->SetAttributeByID(cineware::CRANECAMERA_COMPENSATE_HEADING, Data.GetInt32());
	}
	return Result;
}

// Composes the effect of the CraneCamera attributes into a single transform in the Melange
// coordinate system
FTransform CalculateCraneCameraTransform(const FCraneCameraAttributes& Params)
{
	// We will first construct a transformation in the UnrealEditor coordinate system, as that is
	// easier to visualize and test

	// Local rotation of 90deg around the Y axis in Melange.
	// Will compensate the difference in convention between UnrealEditor (camera shoots out the +X) and
	// C4D (camera shoots out the +Z)
	FTransform Conv = FTransform(FRotator(0, -90, 0), FVector(0, 0, 0));

	// Note: FRotator constructor is Pitch, Yaw and Roll (i.e. Y, Z, X), and these
	// are wrt a camera rotated 90 degrees due to Conv, so a roll will become a pitch, etc
	FTransform Cam  = FTransform(FRotator(0, 0, 0), FVector(0, -Params.CamOffset, 0)) *
					  FTransform(FRotator(-Params.CamBanking, 0, 0), FVector(0, 0, 0)) *
					  FTransform(FRotator(0, 0, Params.CamPitch), FVector(0, 0, 0));

	FTransform Head = FTransform(FRotator(0, 0, 0), FVector(Params.HeadWidth, 0, 0)) *
					  FTransform(FRotator(0, -Params.HeadHeading, 0), FVector(0, 0, 0)) *
					  FTransform(FRotator(0, 0, 0), FVector(0, 0, -Params.HeadHeight));

	FTransform Arm  = FTransform(FRotator(0, 0, 0), FVector(0, -Params.ArmLength, 0)) *
		              FTransform(FRotator(0, 0, Params.ArmPitch), FVector(0, 0, 0));

	FTransform Base = FTransform(FRotator(0, Params.BaseHeading, 0), FVector(0, 0, 0)) *
		              FTransform(FRotator(0, 0, 0), FVector(0, 0, Params.BaseHeight));

	// With Compensate Pitch on, the camera rotates about the end of the arm
	// to compensate the arm pitch, so we need to apply a rotation to undo
	// the effects of the pitch before the arm is accounted for
	if (Params.bCompensatePitch)
	{
		Arm = FTransform(FRotator(0, 0, -Params.ArmPitch), FVector(0, 0, 0)) *
			  Arm;
	}

	// With Compensate Heading on, the camera rotates about the end of the arm
	// to compensate the base's heading, so we need to apply a rotation to undo
	// the effects of the heading before the arm is accounted for
	if (Params.bCompensateHeading)
	{
		Arm = FTransform(FRotator(0, -Params.BaseHeading, 0), FVector(0, 0, 0)) *
			  Arm;
	}

	FTransform FinalTransUE = Conv * Cam * Head * Arm * Base;
	FVector TranslationUE = FinalTransUE.GetTranslation();
	FVector3f EulerUE = (FVector3f)FinalTransUE.GetRotation().Euler();

	// Convert FinalTransUnrealEditor into the cineware coordinate system, so that this can be treated
	// like the other types of animations in ImportAnimations.
	// More specifically, convert them so that that ConvertDirectionLeftHandedYup and
	// the conversion for Ocamera rotations gets them back into UnrealEditor's coordinate system
	// Note: Remember that FRotator's constructor is Pitch, Yaw and Roll (i.e. Y, Z, X)
	return FTransform(FRotator(EulerUE.Y, EulerUE.X, -EulerUE.Z-90),
					  FVector(TranslationUE.X, TranslationUE.Z, -TranslationUE.Y));
}

void FDatasmithC4DImporter::ImportSpline(cineware::SplineObject* SplineActor)
{
	// ActorObject has fewer keys, but uses bezier control points
	// Cache has more keys generated by subdivision, should be parsed with linear interpolation
	cineware::SplineObject* SplineCache = static_cast<cineware::SplineObject*>(GetBestMelangeCache(SplineActor));

	if (SplineActor && SplineCache)
	{
		int32 NumPoints = SplineCache->GetPointCount();
		if (NumPoints < 2)
		{
			return;
		}

		TArray<FRichCurve>& XYZCurves = SplineCurves.FindOrAdd(SplineActor);
		XYZCurves.SetNum(3);
		FRichCurve& XCurve = XYZCurves[0];
		FRichCurve& YCurve = XYZCurves[1];
		FRichCurve& ZCurve = XYZCurves[2];

		float PercentageDenominator = (float)(NumPoints-1);

		// If the spline is closed we have to manually add a final key equal to the first
		if (SplineActor->GetIsClosed())
		{
			// The extra point we manually add will become 1.0
			++PercentageDenominator;
		}

		cineware::Matrix Trans = SplineCache->GetMg();

		const cineware::Vector* Points = SplineCache->GetPointR();
		for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const cineware::Vector& Point = Trans * Points[PointIndex];
			float Percent = PointIndex / PercentageDenominator;
			XCurve.AddKey(Percent, (float)Point.x);
			YCurve.AddKey(Percent, (float)Point.y);
			ZCurve.AddKey(Percent, (float)Point.z);
		}

		if (SplineActor->GetIsClosed())
		{
			const cineware::Vector& FirstPoint = Trans * Points[0];
			XCurve.AddKey(1.0f, (float)FirstPoint.x);
			YCurve.AddKey(1.0f, (float)FirstPoint.y);
			ZCurve.AddKey(1.0f, (float)FirstPoint.z);
		}
	}
}

cineware::BaseObject* FDatasmithC4DImporter::GetBestMelangeCache(cineware::BaseObject* Object)
{
	if (Object == nullptr)
	{
		return nullptr;
	}

	//When primitives types (cube, cone, cylinder...) are exported with the "Save Project for Melange" option,
	//they will have a cache that represent their PolygonObject equivalent.
	cineware::BaseObject* ObjectCache = Object->GetCache();

	//When the primitive has a deformer, the resulting PolygonObject will be in a sub-cache
	if (ObjectCache)
	{
		if (ObjectCache->GetDeformCache())
		{
			ObjectCache = ObjectCache->GetDeformCache();
		}
	}
	else
	{
		ObjectCache = Object->GetDeformCache();
	}

	if (ObjectCache)
	{
		CachesOriginalObject.Add(ObjectCache, Object);
	}

	return ObjectCache;
}

TOptional<FString> FDatasmithC4DImporter::MelangeObjectID(cineware::BaseObject* Object)
{
	//Make sure that Object is not in a cache
	FString HierarchyPosition;
	bool InCache = false;
	cineware::BaseObject* ParentObject = Object;
	while (ParentObject)
	{
		int ObjectHierarchyIndex = 0;
		cineware::BaseObject* PrevObject = ParentObject->GetPred();
		while (PrevObject)
		{
			ObjectHierarchyIndex++;
			PrevObject = PrevObject->GetPred();
		}
		HierarchyPosition = "_" + FString::FromInt(ObjectHierarchyIndex) + HierarchyPosition;

		cineware::BaseObject** OriginalObject = CachesOriginalObject.Find(ParentObject);
		if (OriginalObject)
		{
			InCache = true;
			Object = *OriginalObject;
			ParentObject = Object;
			HierarchyPosition = "_C" + HierarchyPosition;
		}
		else
		{
			ParentObject = ParentObject->GetUp();
		}
	}

	TOptional<FString> MelangeID = GetMelangeBaseList2dID(Object);
	if (MelangeID && InCache)
	{
		MelangeID.GetValue() += HierarchyPosition.Right(HierarchyPosition.Len() - HierarchyPosition.Find("_C", ESearchCase::CaseSensitive) - 2);
	}

	return MelangeID;
}

namespace FC4DImporterImpl
{
	// Returns whether we can remove this actor when optimizing the actor hierarchy
	bool CanRemoveActor(const TSharedPtr<IDatasmithActorElement>& Actor, const TSet<FString>& ActorNamesToKeep, TSharedRef<IDatasmithScene>& DatasmithScene)
	{
		if (Actor->IsA(EDatasmithElementType::Camera | EDatasmithElementType::Light))
		{
			return false;
		}

		if (Actor->IsA(EDatasmithElementType::StaticMeshActor))
		{
			TSharedPtr<IDatasmithMeshActorElement> MeshActor = StaticCastSharedPtr<IDatasmithMeshActorElement>(Actor);
			if (MeshActor->GetStaticMeshPathName() != FString())
			{
				return false;
			}
		}

		if (DatasmithScene->GetMetaData(Actor))
		{
			return false;
		}

		if (ActorNamesToKeep.Contains(FString(Actor->GetName())))
		{
			return false;
		}

		return true;
	}

	void RemoveEmptyActorsRecursive(TSharedPtr<IDatasmithActorElement>& Actor, const TSet<FString>& NamesOfActorsToKeep, TSharedRef<IDatasmithScene>& DatasmithScene)
	{
		// We can't access the parent of a IDatasmithActorElement, so we have to analyze children and remove grandchildren
		// This is also why we need a RootActor in the scene, or else we won't be able to analyze top-level actors
		for (int32 ChildIndex = Actor->GetChildrenCount() - 1; ChildIndex >= 0; --ChildIndex)
		{
			// Have to recurse first or else we will also iterate on our grandchildren
			TSharedPtr<IDatasmithActorElement> Child = Actor->GetChild(ChildIndex);

			RemoveEmptyActorsRecursive(Child, NamesOfActorsToKeep, DatasmithScene);

			// Move grandchildren to children
			if (Child->GetChildrenCount() <= 1 && CanRemoveActor(Child, NamesOfActorsToKeep, DatasmithScene))
			{
				for (int32 GrandChildIndex = Child->GetChildrenCount() - 1; GrandChildIndex >= 0; --GrandChildIndex)
				{
					TSharedPtr<IDatasmithActorElement> GrandChild = Child->GetChild(GrandChildIndex);

					Child->RemoveChild(GrandChild);
					Actor->AddChild(GrandChild);
				}

				Actor->RemoveChild(Child);
			}
		}
	}

	// For now, we can't remove parents of animated nodes because animations are stored wrt local coordinate
	// system. If we optimized an otherwise useless intermediate node, we'd need to bake its transform into all animations of
	// child nodes, which I'm not sure is the ideal behavior as imported animation curves would look very different
	bool KeepParentsOfAnimatedNodes(const TSharedPtr<IDatasmithActorElement>& Actor, TSet<FString>& NamesOfActorsToKeep)
	{
		bool bKeepThisNode = NamesOfActorsToKeep.Contains(Actor->GetName());

		for (int32 ChildIndex = 0; ChildIndex < Actor->GetChildrenCount(); ++ChildIndex)
		{
			bKeepThisNode |= KeepParentsOfAnimatedNodes(Actor->GetChild(ChildIndex), NamesOfActorsToKeep);
		}

		if (bKeepThisNode)
		{
			NamesOfActorsToKeep.Add(Actor->GetName());
		}

		return bKeepThisNode;
	}

	void RemoveEmptyActors(TSharedRef<IDatasmithScene>& DatasmithScene, const TSet<FString>& NamesOfActorsToKeep)
	{
		for (int32 ActorIndex = 0; ActorIndex < DatasmithScene->GetActorsCount(); ++ActorIndex)
		{
			TSharedPtr<IDatasmithActorElement> Actor = DatasmithScene->GetActor(ActorIndex);
			RemoveEmptyActorsRecursive(Actor, NamesOfActorsToKeep, DatasmithScene);
		}
	}
}

namespace FC4DImporterImpl
{
	TSharedPtr<IDatasmithMetaDataElement> CreateMetadataForActor(const TSharedPtr<IDatasmithActorElement>& Actor, const TSharedRef<IDatasmithScene>& DatasmithScene)
	{
		if (!Actor.IsValid())
		{
			return nullptr;
		}

		TSharedPtr<IDatasmithMetaDataElement> Metadata = FDatasmithSceneFactory::CreateMetaData(Actor->GetName());
		Metadata->SetAssociatedElement(Actor);
		DatasmithScene->AddMetaData(Metadata);
		return Metadata;
	}

	void AddMetadataVector(IDatasmithMetaDataElement* Metadata, const FString& Key, const FVector& Value)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
		MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Vector);
		MetadataPropertyPtr->SetValue(*Value.ToString());

		Metadata->AddProperty(MetadataPropertyPtr);
	}

	void AddMetadataColor(IDatasmithMetaDataElement* Metadata, const FString& Key, const FVector& Value)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
		MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Color);
		MetadataPropertyPtr->SetValue(*Value.ToString());

		Metadata->AddProperty(MetadataPropertyPtr);
	}

	void AddMetadataFloat(IDatasmithMetaDataElement* Metadata, const FString& Key, float Value)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
		MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
		MetadataPropertyPtr->SetValue(*LexToString(Value));

		Metadata->AddProperty(MetadataPropertyPtr);
	}

	void AddMetadataTexture(IDatasmithMetaDataElement* Metadata, const FString& Key, const FString& FilePath)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
		MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
		MetadataPropertyPtr->SetValue(*FilePath);

		Metadata->AddProperty(MetadataPropertyPtr);
	}

	void AddMetadataBool(IDatasmithMetaDataElement* Metadata, const FString& Key, bool bValue)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
		MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
		MetadataPropertyPtr->SetValue(bValue? TEXT("True") : TEXT("False"));

		Metadata->AddProperty(MetadataPropertyPtr);
	}

	void AddMetadataString(IDatasmithMetaDataElement* Metadata, const FString& Key, const FString& Value)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*Key);
		MetadataPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::String);
		MetadataPropertyPtr->SetValue(*Value);

		Metadata->AddProperty(MetadataPropertyPtr);
	}

	void ImportActorMetadata(cineware::BaseObject* Object, const TSharedPtr<IDatasmithActorElement>& Actor, TSharedRef<IDatasmithScene>& DatasmithScene)
	{
		cineware::DynamicDescription* DynamicDescription = Object->GetDynamicDescription();
		if (!DynamicDescription)
		{
			return;
		}

		TSharedPtr<IDatasmithMetaDataElement> Metadata = nullptr;

		void* BrowserHandle = DynamicDescription->BrowseInit();
		cineware::DescID DescId;
		const cineware::BaseContainer* DescContainer;
		while (DynamicDescription->BrowseGetNext(BrowserHandle, &DescId, &DescContainer))
		{
			if (DescId[0].id != cineware::ID_USERDATA)
			{
				continue;;
			}

			if (!Metadata.IsValid())
			{
				Metadata = CreateMetadataForActor(Actor, DatasmithScene);
				if (!Metadata.IsValid())
				{
					return;
				}
			}

			cineware::GeData Data;
			if (!Object->GetParameter(DescId, Data))
			{
				continue;
			}

			FString DataName = MelangeStringToFString(DescContainer->GetString(cineware::DESC_NAME));

			cineware::Int32 UserDataType = DescContainer->GetInt32(cineware::DESC_CUSTOMGUI);
			if (UserDataType == cineware::DA_VECTOR)
			{
				FVector ConvertedVector = ConvertMelangePosition(Data.GetVector());
				AddMetadataVector(Metadata.Get(), *DataName, ConvertedVector);
			}
			else if (UserDataType == cineware::DA_REAL)
			{
				AddMetadataFloat(Metadata.Get(), *DataName, static_cast<float>(Data.GetFloat()));
			}
			else if (UserDataType == 1000492 /*color*/)
			{
				AddMetadataColor(Metadata.Get(), *DataName, MelangeVectorToFVector(Data.GetVector()));
			}
			else if (UserDataType == 1000484 /*texture*/)
			{
				AddMetadataTexture(Metadata.Get(), *DataName, *GeDataToString(Data));
			}
			else if (UserDataType == 400006001 /*boolean*/)
			{
				AddMetadataBool(Metadata.Get(), *DataName, Data.GetInt32() != 0);
			}
			else
			{
				FString ValueString = GeDataToString(Data);
				if (!ValueString.IsEmpty())
				{
					AddMetadataString(Metadata.Get(), *DataName, *ValueString);
				}
			}
		}
		DynamicDescription->BrowseFree(BrowserHandle);
	}
}

bool FDatasmithC4DImporter::AddChildActor(cineware::BaseObject* Object, TSharedPtr<IDatasmithActorElement> ParentActor, cineware::Matrix WorldTransformMatrix, const TSharedPtr<IDatasmithActorElement>& Actor)
{
	FC4DImporterImpl::ImportActorMetadata(Object, Actor, DatasmithScene);

	if (NamesOfAllActors.Contains(Actor->GetName()))
	{
		//Duplicate name, don't import twice.
		return false;
	}
	NamesOfAllActors.Add(Actor->GetName());

	ActorElementToAnimationSources.Add(Actor.Get(), Object);

	if (Object->GetType() == Ocamera || Object->GetType() == Olight)
	{
		// Compensates the fact that in C4D cameras/lights shoot out towards +Z, while in
		// UnrealEditor they shoot towards +X
		cineware::Matrix CameraRotation(
			cineware::Vector(0.0, 0.0, 0.0),
			cineware::Vector(0.0, 0.0, 1.0),
			cineware::Vector(0.0, 1.0, 0.0),
			cineware::Vector(-1.0, 0.0, 0.0));
		WorldTransformMatrix = WorldTransformMatrix * CameraRotation;
	}

	// Convert to a float array so we can use Imath
	float FloatMatrix[16];
	FloatMatrix[0]  = static_cast<float>(WorldTransformMatrix.v1.x);
	FloatMatrix[1]  = static_cast<float>(WorldTransformMatrix.v1.y);
	FloatMatrix[2]  = static_cast<float>(WorldTransformMatrix.v1.z);
	FloatMatrix[3]  = 0;
	FloatMatrix[4]  = static_cast<float>(WorldTransformMatrix.v2.x);
	FloatMatrix[5]  = static_cast<float>(WorldTransformMatrix.v2.y);
	FloatMatrix[6]  = static_cast<float>(WorldTransformMatrix.v2.z);
	FloatMatrix[7]  = 0;
	FloatMatrix[8]  = static_cast<float>(WorldTransformMatrix.v3.x);
	FloatMatrix[9]  = static_cast<float>(WorldTransformMatrix.v3.y);
	FloatMatrix[10] = static_cast<float>(WorldTransformMatrix.v3.z);
	FloatMatrix[11] = 0;
	FloatMatrix[12] = static_cast<float>(WorldTransformMatrix.off.x);
	FloatMatrix[13] = static_cast<float>(WorldTransformMatrix.off.y);
	FloatMatrix[14] = static_cast<float>(WorldTransformMatrix.off.z);
	FloatMatrix[15] = 1.0;

	// We use Imath::extractAndRemoveScalingAndShear() because FMatrix::ExtractScaling() is deemed unreliable.
	// Set up a scaling and rotation matrix.
	Imath::Matrix44<float> Matrix(FloatMatrix[0], FloatMatrix[1], FloatMatrix[2],  0.0,
		FloatMatrix[4], FloatMatrix[5], FloatMatrix[6],  0.0,
		FloatMatrix[8], FloatMatrix[9], FloatMatrix[10], 0.0,
		0.0,            0.0,             0.0, 1.0);

	// Remove any scaling from the matrix and get the scale vector that was initially present.
	Imath::Vec3<float> Scale(1.0f, 1.0f, 1.0f);
	Imath::Vec3<float> Shear(0.0f, 0.0f, 0.0f);
	bool bExtracted = Imath::extractAndRemoveScalingAndShear<float>(Matrix, Scale, Shear, false);
	if (!bExtracted)
	{
		UE_LOGF(LogDatasmithC4DImport, Warning, "Actor %ls (%ls) has some zero scaling", Actor->GetName(), Actor->GetLabel());

		// extractAndRemoveScalingAndShear may have partially written to these vectors, so we need to
		// reset them here to make sure they're valid for code below
		Scale = Imath::Vec3<float>(1.0f, 1.0f, 1.0f);
		Shear = Imath::Vec3<float>(0.0f, 0.0f, 0.0f);
	}

	// Initialize a rotation quaternion with the rotation matrix.
	Imath::Quat<float> Quaternion = Imath::extractQuat<float>(Matrix);

	// Switch Z and Y axes for the scale due to coordinate system conversions
	FVector WorldScale = FVector(Scale.x, Scale.z, Scale.y);

	// Convert the left-handed Y-up coordinate rotation into an Unreal left-handed Z-up coordinate rotation.
	// This is done by doing a 90 degree rotation about the X axis.
	float Y = Quaternion.v.y;
	float Z = Quaternion.v.z;
	Quaternion.v.y = -Z;
	Quaternion.v.z =  Y;
	Quaternion.normalize();

	// Make sure Unreal will be able to handle the rotation quaternion.
	float              Angle = Quaternion.angle();
	Imath::Vec3<float> Axis  = Quaternion.axis();
	FQuat WorldRotation = FQuat(FVector(Axis.x, Axis.y, Axis.z), Angle);

	// Scale and convert the world transform translation into a Datasmith actor world translation.
	FVector WorldTranslation = ConvertMelangePosition(FVector(FloatMatrix[12], FloatMatrix[13], FloatMatrix[14]));

	// Remove our children or else the ConvertChildsToRelative + ConvertChildsToWorld combo within SetTranslation/Rotation/Scale will
	// cause our children to maintain their relative transform to Actor, which is not what we want. When we set a Trans/Rot/Scale we
	// are setting the final, absolute world space value
	int32 ChildCount = Actor->GetChildrenCount();
	TArray<TSharedPtr<IDatasmithActorElement>> Children;
	Children.SetNum(ChildCount);
	for (int32 ChildIndex = ChildCount - 1; ChildIndex >= 0; --ChildIndex)
	{
		const TSharedPtr<IDatasmithActorElement>& Child = Actor->GetChild(ChildIndex);

		Children[ChildIndex] = Child;
		Actor->RemoveChild(Child);
	}

	Actor->SetTranslation(WorldTranslation);
	Actor->SetScale(WorldScale);
	Actor->SetRotation(WorldRotation);

	ParentActor->AddChild(Actor);
	for (const TSharedPtr<IDatasmithActorElement>& Child : Children)
	{
		Actor->AddChild(Child, EDatasmithActorAttachmentRule::KeepWorldTransform);
	}

	return true;
}

TSharedPtr<IDatasmithActorElement> FDatasmithC4DImporter::ImportNullActor(cineware::BaseObject* Object, const FString& DatasmithName, const FString& DatasmithLabel)
{
	TSharedPtr<IDatasmithActorElement> ActorElement = FDatasmithSceneFactory::CreateActor(*DatasmithName);
	ActorElement->SetLabel(*DatasmithLabel);
	return ActorElement;
}

// workaround define from UE that conflicts with C4D SDK constant
#ifdef LIGHT_TYPE_SPOT
#undef LIGHT_TYPE_SPOT
#endif

namespace
{
	TSharedPtr<IDatasmithLightActorElement> CreateDatasmithLightActorElement(int32 MelangeLightTypeId, const FString& Name, const FString& Label)
	{
		TSharedPtr<IDatasmithLightActorElement> Result = nullptr;
		switch (MelangeLightTypeId)
		{
		case cineware::LIGHT_TYPE_OMNI:
			Result = FDatasmithSceneFactory::CreatePointLight(*Name);
			break;
		case cineware::LIGHT_TYPE_SPOT:
			Result = FDatasmithSceneFactory::CreateSpotLight(*Name);
			break;
		case cineware::LIGHT_TYPE_SPOTRECT:
			Result = FDatasmithSceneFactory::CreateSpotLight(*Name);
			break;
		case cineware::LIGHT_TYPE_DISTANT:
			Result = FDatasmithSceneFactory::CreateDirectionalLight(*Name);
			break;
		case cineware::LIGHT_TYPE_PARALLEL:
			Result = FDatasmithSceneFactory::CreateSpotLight(*Name);
			break;
		case cineware::LIGHT_TYPE_PARSPOTRECT:
			Result = FDatasmithSceneFactory::CreateSpotLight(*Name);
			break;
		case cineware::LIGHT_TYPE_TUBE:
			Result = FDatasmithSceneFactory::CreateSpotLight(*Name);
			break;
		case cineware::LIGHT_TYPE_AREA:
			Result = FDatasmithSceneFactory::CreateAreaLight(*Name);
			break;
		case cineware::LIGHT_TYPE_PHOTOMETRIC:
			Result = FDatasmithSceneFactory::CreatePointLight(*Name);
			break;
		default:
			break;
		}

		if (Result.IsValid())
		{
			Result->SetLabel(*Label);
		}

		return Result;
	}

	EDatasmithLightUnits GetDatasmithLightIntensityUnits(int32 MelangeLightUnitId)
	{
		switch (MelangeLightUnitId)
		{
		case cineware::LIGHT_PHOTOMETRIC_UNIT_LUMEN:
			return EDatasmithLightUnits::Lumens;
		case cineware::LIGHT_PHOTOMETRIC_UNIT_CANDELA:
			return EDatasmithLightUnits::Candelas;
		default:
			break;
		}

		return EDatasmithLightUnits::Unitless;
	}

	// Is called when a LightType is Ligth Area to fits its shape
	EDatasmithLightShape GetDatasmithAreaLightShape(int32 AreaLightC4DId)
	{
		switch (AreaLightC4DId)
		{
		case cineware::LIGHT_AREADETAILS_SHAPE_DISC:
			return EDatasmithLightShape::Disc;
		case cineware::LIGHT_AREADETAILS_SHAPE_RECTANGLE:
			return EDatasmithLightShape::Rectangle;
		case cineware::LIGHT_AREADETAILS_SHAPE_SPHERE:
			return EDatasmithLightShape::Sphere;
		case cineware::LIGHT_AREADETAILS_SHAPE_CYLINDER:
			return EDatasmithLightShape::Cylinder;
		case cineware::LIGHT_AREADETAILS_SHAPE_CUBE:
			return EDatasmithLightShape::Rectangle;
		case cineware::LIGHT_AREADETAILS_SHAPE_HEMISPHERE:
			return EDatasmithLightShape::Sphere;
		case cineware::LIGHT_AREADETAILS_SHAPE_OBJECT:
			return EDatasmithLightShape::None;
		case cineware::LIGHT_AREADETAILS_SHAPE_LINE:
			return EDatasmithLightShape::Cylinder;
		case cineware::LIGHT_AREADETAILS_SHAPE_PCYLINDER:
			return EDatasmithLightShape::Cylinder;
		default:
			break;
		}

		return EDatasmithLightShape::None;
	}
}

namespace
{
	cineware::Int32 MelangeColorProfile = cineware::DOCUMENT_COLORPROFILE_SRGB;

	FVector ToLinearColor(const FVector& Color)
	{
		// Document is already linear, nothing to do
		if (MelangeColorProfile == cineware::DOCUMENT_COLORPROFILE_LINEAR)
		{
			return Color;
		}

		// The default seems to be sRGB
		FLinearColor ActuallyLinearColor = FLinearColor(FLinearColor(Color).QuantizeRound());
		return FVector(ActuallyLinearColor.R, ActuallyLinearColor.G, ActuallyLinearColor.B);
	}

	// Gets a color weighted by its brightness
	FVector MelangeGetLayerColor(cineware::BaseList2D* MelangeObject, cineware::Int32 ColorAttributeID, cineware::Int32 BrightnessAttributeID)
	{
		FVector Result = FVector::ZeroVector;
		if (MelangeObject)
		{
			float Brightness = MelangeGetFloat(MelangeObject, BrightnessAttributeID);
			FVector Color = MelangeGetVector(MelangeObject, ColorAttributeID);
			Result = ToLinearColor(Color * Brightness);
		}
		return Result;
	}

	// In here instead of utils because it depends on the document color profile
	FVector MelangeGetColor(cineware::BaseList2D* MelangeObject, cineware::Int32 MelangeDescId)
	{
		FVector Result = FVector::ZeroVector;
		if (MelangeObject)
		{
			Result = ToLinearColor(MelangeGetVector(MelangeObject, MelangeDescId));
		}
		return Result;
	}

	void AddColorToMaterial(const TSharedPtr<IDatasmithMaterialInstanceElement>& Material, const FString& DatasmithPropName, const FLinearColor& LinearColor)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*DatasmithPropName);
		MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Color);
		MaterialPropertyPtr->SetValue(*LinearColor.ToString());

		Material->AddProperty(MaterialPropertyPtr);
	}

	void AddFloatToMaterial(const TSharedPtr<IDatasmithMaterialInstanceElement>& Material, const FString& DatasmithPropName, float Value)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*DatasmithPropName);
		MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
		MaterialPropertyPtr->SetValue(*LexToString(Value));

		Material->AddProperty(MaterialPropertyPtr);
	}

	void AddBoolToMaterial(const TSharedPtr<IDatasmithMaterialInstanceElement>& Material, const FString& DatasmithPropName, bool bValue)
	{
		TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*DatasmithPropName);
		MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
		MaterialPropertyPtr->SetValue(bValue ? TEXT("True") : TEXT("False"));

		Material->AddProperty(MaterialPropertyPtr);
	}

	void AddTextureToMaterial(const TSharedPtr<IDatasmithMaterialInstanceElement>& Material, const FString& DatasmithPropName, TSharedPtr<IDatasmithTextureElement> Texture)
	{
		if (Texture == nullptr)
		{
			return;
		}

		TSharedPtr<IDatasmithKeyValueProperty> MaterialPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty(*DatasmithPropName);
		MaterialPropertyPtr->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
		MaterialPropertyPtr->SetValue(Texture->GetName());

		Material->AddProperty(MaterialPropertyPtr);
	}
}

TSharedPtr<IDatasmithLightActorElement> FDatasmithC4DImporter::ImportLight(cineware::BaseObject* InC4DLightPtr, const FString& DatasmithName, const FString& DatasmithLabel)
{
	cineware::GeData C4DData;

	// Actor type
	int32 LightTypeId = MelangeGetInt32(InC4DLightPtr, cineware::LIGHT_TYPE);
	TSharedPtr<IDatasmithLightActorElement> LightActor = CreateDatasmithLightActorElement(LightTypeId, DatasmithName, DatasmithLabel);
	if (!LightActor.IsValid())
	{
		UE_LOGF(LogDatasmithC4DImport, Warning, "Failed to create DatasmithLightActorElement for light '%ls'", *MelangeObjectName(InC4DLightPtr));
		return nullptr;
	}

	// Color
	FLinearColor Color = FLinearColor(MelangeGetColor(InC4DLightPtr, cineware::LIGHT_COLOR));

	// Temperature
	bool bUseTemperature = MelangeGetBool(InC4DLightPtr, cineware::LIGHT_TEMPERATURE);
	double Temperature = MelangeGetDouble(InC4DLightPtr, cineware::LIGHT_TEMPERATURE_MAIN);
	if (Temperature == 0)
	{
		Temperature = 6500.0;
	}

	// Intensity and units
	double Intensity = 1.0;
	EDatasmithLightUnits Units = EDatasmithLightUnits::Unitless;
	if (MelangeGetBool(InC4DLightPtr, cineware::LIGHT_PHOTOMETRIC_UNITS))
	{
		Units = GetDatasmithLightIntensityUnits(MelangeGetInt32(InC4DLightPtr, cineware::LIGHT_PHOTOMETRIC_UNIT));

		Intensity = MelangeGetDouble(InC4DLightPtr, cineware::LIGHT_PHOTOMETRIC_INTENSITY); // Cd/lm value in 'Photometric' tab
	}

	// Brightness
	Intensity *= MelangeGetDouble(InC4DLightPtr, cineware::LIGHT_BRIGHTNESS); // percentage value on 'General' tab, usually = 1.0
	if (Units == EDatasmithLightUnits::Unitless)
	{
		if (LightActor->IsA(EDatasmithElementType::PointLight))
		{
			Intensity *= UnitlessIESandPointLightIntensity;
		}
		else
		{
			Intensity *= UnitlessGlobalLightIntensity;
		}
	}

	// IES light
	// Checks if "Photometric Data" is enabled
	// Apparently non-IES lights can have this checked while the checkbox is in a "disabled state", so we must also check the light type
	FString IESPath;
	TOptional<double> IESBrightnessScale;
	bool bUseIES = LightTypeId == cineware::LIGHT_TYPE_PHOTOMETRIC && MelangeGetBool(InC4DLightPtr, cineware::LIGHT_PHOTOMETRIC_DATA);
	if (bUseIES)
	{
		FString IESFilename = MelangeGetString(InC4DLightPtr, cineware::LIGHT_PHOTOMETRIC_FILE);

		IESPath = SearchForFile(IESFilename, C4dDocumentFilename);
		if (IESPath.IsEmpty())
		{
			bUseIES = false;
			UE_LOGF(LogDatasmithC4DImport, Warning, "Could not find IES file '%ls' used by light '%ls'", *IESFilename, *MelangeObjectName(InC4DLightPtr));
		}
		else
		{
			// Create IES texture
			const FString BaseFilename = FPaths::GetBaseFilename(IESPath);
			FString TextureName = FDatasmithUtils::SanitizeObjectName(BaseFilename + TEXT("_IES"));
			TSharedPtr<IDatasmithTextureElement> Texture = FDatasmithSceneFactory::CreateTexture(*TextureName);
			Texture->SetTextureMode(EDatasmithTextureMode::Ies);
			Texture->SetLabel(*BaseFilename);
			Texture->SetFile(*IESPath);
			DatasmithScene->AddTexture(Texture);

			// Set IES attributes
			LightActor->SetUseIesBrightness(Units == EDatasmithLightUnits::Unitless);
			LightActor->SetIesTexturePathName(*TextureName);
		}

	}

	// Set common parameters for all lights (including directional lights)
	LightActor->SetIntensity(Intensity);
	LightActor->SetUseIes(bUseIES);
	LightActor->SetTemperature(Temperature);
	LightActor->SetUseTemperature(bUseTemperature);
	LightActor->SetColor(Color);

	// Set point light parameters
	if (LightActor->IsA(EDatasmithElementType::PointLight))
	{
		TSharedPtr<IDatasmithPointLightElement> PointLightActor = StaticCastSharedPtr<IDatasmithPointLightElement>(LightActor);

		PointLightActor->SetIntensityUnits(Units);

		// Attenuation radius
		int32 FalloffOption = MelangeGetInt32(InC4DLightPtr, cineware::LIGHT_DETAILS_FALLOFF);
		if (FalloffOption == cineware::LIGHT_DETAILS_FALLOFF_NONE)
		{
			PointLightActor->SetAttenuationRadius(16384.0f); // Seems to be the maximum value for the slider in the details panel
		}
		// TODO: Add support for other falloff types
		else
		{
			PointLightActor->SetAttenuationRadius(MelangeGetFloat(InC4DLightPtr, cineware::LIGHT_DETAILS_OUTERDISTANCE));
		}
	}

	// Set spot light parameters
	if (LightActor->IsA(EDatasmithElementType::SpotLight))
	{
		TSharedPtr<IDatasmithSpotLightElement> SpotLightActor = StaticCastSharedPtr<IDatasmithSpotLightElement>(LightActor);

		// Inner angle
		float LightInnerAngleInRadians = MelangeGetFloat(InC4DLightPtr, cineware::LIGHT_DETAILS_INNERANGLE);
		SpotLightActor->SetInnerConeAngle((FMath::RadiansToDegrees(LightInnerAngleInRadians) * 90) / 175);

		// Outer angle
		float LightOuterAngleInRadians = MelangeGetFloat(InC4DLightPtr, cineware::LIGHT_DETAILS_OUTERANGLE);
		SpotLightActor->SetOuterConeAngle((FMath::RadiansToDegrees(LightOuterAngleInRadians) * 90) / 175);
	}

	// Set area light parameters
	if (LightActor->IsA(EDatasmithElementType::AreaLight))
	{
		TSharedPtr<IDatasmithAreaLightElement> AreaLightActor = StaticCastSharedPtr<IDatasmithAreaLightElement>(LightActor);

		// Area width
		AreaLightActor->SetWidth(MelangeGetFloat(InC4DLightPtr, cineware::LIGHT_AREADETAILS_SIZEX));

		// Area length
		AreaLightActor->SetLength(MelangeGetFloat(InC4DLightPtr, cineware::LIGHT_AREADETAILS_SIZEY));

		// Area shape and type
		EDatasmithLightShape AreaShape = GetDatasmithAreaLightShape(MelangeGetInt32(InC4DLightPtr, cineware::LIGHT_AREADETAILS_SHAPE));

		// AreaLightType will default to Point, which is OK for most shapes except the planar shapes like Disc and Rectangle.
		// Also, if the user enabled the "Z Direction Only" checkbox we'll also use Rect type as the Point type is omnidirectional
		EDatasmithAreaLightType AreaType = EDatasmithAreaLightType::Point;
		bool bOnlyZ = MelangeGetBool(InC4DLightPtr, cineware::LIGHT_DETAILS_ONLYZ);
		if (bOnlyZ || AreaShape == EDatasmithLightShape::Rectangle || AreaShape == EDatasmithLightShape::Disc)
		{
			AreaType = EDatasmithAreaLightType::Rect;
		}

		AreaLightActor->SetLightType(AreaType);
		AreaLightActor->SetLightShape(AreaShape);
	}

	return LightActor;
}

TSharedPtr<IDatasmithCameraActorElement> FDatasmithC4DImporter::ImportCamera(cineware::BaseObject* InC4DCameraPtr, const FString& DatasmithName, const FString& DatasmithLabel)
{
	TSharedPtr<IDatasmithCameraActorElement> CameraActor = FDatasmithSceneFactory::CreateCameraActor(*DatasmithName);
	CameraActor->SetLabel(*DatasmithLabel);

	cineware::GeData C4DData;

	cineware::BaseTag* LookAtTag = InC4DCameraPtr->GetTag(Ttargetexpression);
	cineware::BaseList2D* LookAtObject = LookAtTag ? MelangeGetLink(LookAtTag, cineware::TARGETEXPRESSIONTAG_LINK) : nullptr;
	if (LookAtObject)
	{
		//LookAtObject can not be a cached object or an instanced object so GetMelangeBaseList2dID should be the final ID
		TOptional<FString> LookAtID = GetMelangeBaseList2dID(LookAtObject);
		if (!LookAtID)
		{
			return TSharedPtr<IDatasmithCameraActorElement>();
		}
		CameraActor->SetLookAtActor(*(LookAtID.GetValue()));
		CameraActor->SetLookAtAllowRoll(true);
		NamesOfActorsToKeep.Add(LookAtID.GetValue());
	}

	float CameraFocusDistanceInCM = MelangeGetFloat(InC4DCameraPtr, cineware::CAMERAOBJECT_TARGETDISTANCE);
	CameraActor->SetFocusDistance(CameraFocusDistanceInCM);

	float CameraFocalLengthMilimeters = MelangeGetFloat(InC4DCameraPtr, cineware::CAMERA_FOCUS);
	CameraActor->SetFocalLength(CameraFocalLengthMilimeters);

	float CameraHorizontalFieldOfViewInDegree = FMath::RadiansToDegrees(MelangeGetFloat(InC4DCameraPtr, cineware::CAMERAOBJECT_FOV));
	float CameraSensorWidthInMillimeter = 2 * (CameraFocalLengthMilimeters * tan((0.5f * CameraHorizontalFieldOfViewInDegree) / 57.296f));
	CameraActor->SetSensorWidth(CameraSensorWidthInMillimeter);

	// Set the camera aspect ratio (width/height).
	cineware::RenderData* SceneRenderer = C4dDocument->GetActiveRenderData();
	cineware::Float AspectRatioOfRenderer, RendererWidth, RendererHeight, PixelAspectRatio;
	SceneRenderer->GetResolution(RendererWidth, RendererHeight, PixelAspectRatio, AspectRatioOfRenderer);
	double AspectRatio = RendererWidth / RendererHeight;
	CameraActor->SetSensorAspectRatio(static_cast<float>(AspectRatio));

	// We only use manual exposure control with aperture, shutter speed and ISO if the exposure checkbox is enabled
	// Aperture is always used for depth of field effects though, which is why its outside of this
	if (MelangeGetBool(InC4DCameraPtr, cineware::CAMERAOBJECT_EXPOSURE))
	{
		float ShutterSpeed = MelangeGetFloat(InC4DCameraPtr, cineware::CAMERAOBJECT_SHUTTER_SPEED_VALUE);
		CameraActor->GetPostProcess()->SetCameraShutterSpeed(ShutterSpeed ? 1.0f/ShutterSpeed : -1.0f);

		float ISO = MelangeGetFloat(InC4DCameraPtr, cineware::CAMERAOBJECT_ISO_VALUE);
		CameraActor->GetPostProcess()->SetCameraISO(ISO ? ISO : -1.0f);
	}
	float Aperture = MelangeGetFloat(InC4DCameraPtr, cineware::CAMERAOBJECT_FNUMBER_VALUE);
	CameraActor->SetFStop(Aperture ? Aperture : -1.0f);

	cineware::BaseTag* Tag = InC4DCameraPtr->GetFirstTag();
	while (Tag)
	{
		cineware::Int32 TagType = Tag->GetType();
		if (TagType == Tcrane)
		{
			TSharedRef<FCraneCameraAttributes> Attributes = ExtractCraneCameraAttributes(Tag);
			CraneCameraToAttributes.Add(InC4DCameraPtr, Attributes);
			break;
		}
		Tag = Tag->GetNext();
	}

	return CameraActor;
}

TSharedPtr<IDatasmithTextureElement> FDatasmithC4DImporter::ImportTexture(const FString& TexturePath, EDatasmithTextureMode TextureMode)
{
	if (TexturePath.IsEmpty())
	{
		return nullptr;
	}

	const TCHAR* TexturePathChr = *TexturePath;
	if (TexturePathChr == nullptr)
	{
		return nullptr;
	}

	const FString TextureHash = FMD5::HashAnsiString(TexturePathChr);
	const FString TextureName = FString::Printf(TEXT("%ls_%d"), *TextureHash, int32(TextureMode));
	if (TSharedPtr<IDatasmithTextureElement>* FoundImportedTexture = ImportedTextures.Find(TextureName))
	{
		return *FoundImportedTexture;
	}

	TSharedPtr<IDatasmithTextureElement> Texture = FDatasmithSceneFactory::CreateTexture(*TextureName);
	Texture->SetTextureMode(TextureMode);
	Texture->SetLabel(*FPaths::GetBaseFilename(TexturePath));
	Texture->SetFile(*TexturePath);
	DatasmithScene->AddTexture(Texture);

	return Texture;
}

FString FDatasmithC4DImporter::GetBaseShaderTextureFilePath(cineware::BaseList2D* BaseShader)
{
	FString TextureFilePath;

	while (BaseShader && TextureFilePath.IsEmpty())
	{
		switch (BaseShader->GetType())
		{
		case Xbitmap:
		{
			FString Filepath = MelangeFilenameToPath(static_cast<cineware::BaseShader*>(BaseShader)->GetFileName());
			TextureFilePath = SearchForFile(Filepath, C4dDocumentFilename);
			break;
		}
		default:
			TextureFilePath = GetBaseShaderTextureFilePath(static_cast<cineware::BaseShader*>(BaseShader)->GetDown());
			break;
		}

		BaseShader = BaseShader->GetNext();
	}

	return TextureFilePath;
}

TSharedPtr<IDatasmithMaterialInstanceElement> FDatasmithC4DImporter::ImportMaterial(cineware::Material* InC4DMaterialPtr)
{
	TOptional<FString> DatasmithName = GetMelangeBaseList2dID(InC4DMaterialPtr);
	if (!DatasmithName)
	{
		return TSharedPtr<IDatasmithMaterialInstanceElement>();
	}
	FString DatasmithLabel = FDatasmithUtils::SanitizeObjectName(MelangeObjectName(InC4DMaterialPtr));

	TSharedPtr<IDatasmithMaterialInstanceElement> MaterialPtr = FDatasmithSceneFactory::CreateMaterialInstance(*(DatasmithName.GetValue()));
	MaterialPtr->SetLabel(*DatasmithLabel);
	MaterialPtr->SetMaterialType(EDatasmithReferenceMaterialType::Opaque);

	// Color
	bool bUseColor = InC4DMaterialPtr->GetChannelState(CHANNEL_COLOR);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_Color"), bUseColor);
	if (bUseColor)
	{
		FLinearColor Color = FLinearColor(MelangeGetLayerColor(InC4DMaterialPtr, cineware::MATERIAL_COLOR_COLOR, cineware::MATERIAL_COLOR_BRIGHTNESS));
		AddColorToMaterial(MaterialPtr, TEXT("Color"), Color);

		cineware::BaseList2D* MaterialShader = MelangeGetLink(InC4DMaterialPtr, cineware::MATERIAL_COLOR_SHADER);
		FString TextureFilePath = GetBaseShaderTextureFilePath(MaterialShader);
		TSharedPtr<IDatasmithTextureElement> ColorMap = ImportTexture(TextureFilePath, EDatasmithTextureMode::Diffuse);
		AddTextureToMaterial(MaterialPtr, TEXT("ColorMap"), ColorMap);

		bool bUseColorMap = !TextureFilePath.IsEmpty();
		AddBoolToMaterial(MaterialPtr, TEXT("Use_ColorMap"), bUseColorMap);
		if (bUseColorMap)
		{
			AddFloatToMaterial(MaterialPtr, TEXT("Exposure"), 0);

			// Check for the good type of Texture Mixing and Blending
			int32 MixingTypeId = MelangeGetInt32(InC4DMaterialPtr, cineware::MATERIAL_COLOR_TEXTUREMIXING);
			switch (MixingTypeId)
			{
			case cineware::MATERIAL_TEXTUREMIXING_ADD:
				AddBoolToMaterial(MaterialPtr, TEXT("MixMode_Add"), true);
				break;
			case cineware::MATERIAL_TEXTUREMIXING_SUBTRACT:
				AddBoolToMaterial(MaterialPtr, TEXT("MixMode_Subtract"), true);
				break;
			case cineware::MATERIAL_TEXTUREMIXING_MULTIPLY:
				AddBoolToMaterial(MaterialPtr, TEXT("MixMode_Multiply"), true);
				break;
			default: // MATERIAL_TEXTUREMIXING_NORMAL
				AddBoolToMaterial(MaterialPtr, TEXT("MixMode_Normal"), true);
				break;
			}

			float MixStrength = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_COLOR_TEXTURESTRENGTH);
			AddFloatToMaterial(MaterialPtr, TEXT("Mix_Strength"), MixStrength);
		}
	}

	// Emissive
	bool bUseEmissive = InC4DMaterialPtr->GetChannelState(CHANNEL_LUMINANCE);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_Emissive"), bUseEmissive);
	if (bUseEmissive)
	{
		float EmissiveGlowStrength = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_LUMINANCE_BRIGHTNESS);
		AddFloatToMaterial(MaterialPtr, TEXT("Emissive_Glow_Strength"), EmissiveGlowStrength);

		FLinearColor EmissiveColor = FLinearColor(MelangeGetColor(InC4DMaterialPtr, cineware::MATERIAL_LUMINANCE_COLOR));
		AddColorToMaterial(MaterialPtr, TEXT("Emissive_Color"), EmissiveColor);

		cineware::BaseList2D* LuminanceShader = MelangeGetLink(InC4DMaterialPtr, cineware::MATERIAL_LUMINANCE_SHADER);
		FString LuminanceFilePath = GetBaseShaderTextureFilePath(LuminanceShader);
		TSharedPtr<IDatasmithTextureElement> EmissiveMap = ImportTexture(LuminanceFilePath, EDatasmithTextureMode::Other);
		AddTextureToMaterial(MaterialPtr, TEXT("Emissive_Map"), EmissiveMap);

		bool bUseEmissiveMap = !LuminanceFilePath.IsEmpty();
		AddBoolToMaterial(MaterialPtr, TEXT("Use_EmissiveMap"), bUseEmissiveMap);
		if (bUseEmissiveMap)
		{
			float EmissiveMapExposure = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_LUMINANCE_TEXTURESTRENGTH);
			AddFloatToMaterial(MaterialPtr, TEXT("Emissive_Map_Exposure"), EmissiveMapExposure);
		}
	}

	// Transparency
	bool bUseTransparency = InC4DMaterialPtr->GetChannelState(CHANNEL_TRANSPARENCY);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_Transparency"), bUseTransparency);
	if (bUseTransparency)
	{
		MaterialPtr->SetMaterialType(EDatasmithReferenceMaterialType::Transparent);

		cineware::BaseList2D* TransparencyShader = MelangeGetLink(InC4DMaterialPtr, cineware::MATERIAL_TRANSPARENCY_SHADER);
		FString TransparencyMapPath = GetBaseShaderTextureFilePath(TransparencyShader);
		TSharedPtr<IDatasmithTextureElement> TransparencyMap = ImportTexture(TransparencyMapPath, EDatasmithTextureMode::Other);
		AddTextureToMaterial(MaterialPtr, TEXT("Transparency_Map"), TransparencyMap);

		bool bUseTransparencyMap = !TransparencyMapPath.IsEmpty();
		AddBoolToMaterial(MaterialPtr, TEXT("Use_TransparencyMap"), bUseTransparencyMap);
		if (bUseTransparencyMap)
		{
			float TextureStrength = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_TRANSPARENCY_TEXTURESTRENGTH);
			AddFloatToMaterial(MaterialPtr, TEXT("TransparencyMap_Amount"), TextureStrength);
		}
		else
		{
			float BrightnessValue = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_TRANSPARENCY_BRIGHTNESS);
			FVector3f TransparencyColor = (FVector3f)MelangeGetVector(InC4DMaterialPtr, cineware::MATERIAL_TRANSPARENCY_COLOR);

			// In Cinema4D Transparency Color seems to be used just as another multiplier for the opacity, not as an actual color
			AddFloatToMaterial(MaterialPtr, TEXT("Transparency_Amount"), BrightnessValue * TransparencyColor.X * TransparencyColor.Y * TransparencyColor.Z);
		}

		float TransparencyRefraction = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_TRANSPARENCY_REFRACTION);
		AddFloatToMaterial(MaterialPtr, TEXT("Transparency_Refraction"), TransparencyRefraction);
	}

	cineware::GeData C4DData;

	// Specular
	bool bUseSpecular = InC4DMaterialPtr->GetChannelState(CHANNEL_REFLECTION);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_Specular"), bUseSpecular);
	if (bUseSpecular)
	{
		cineware::Int32 ReflectionLayerCount = InC4DMaterialPtr->GetReflectionLayerCount();
		if (ReflectionLayerCount > 0)
		{
			bool bUseReflectionColor = false;

			// Grab the total base color from all diffuse layers
			FVector ReflectionColor(0, 0, 0);
			for (int32 LayerIndex = ReflectionLayerCount - 1; LayerIndex >= 0; --LayerIndex)
			{
				cineware::ReflectionLayer* ReflectionLayer = InC4DMaterialPtr->GetReflectionLayerIndex(LayerIndex);
				if (ReflectionLayer == nullptr)
				{
					continue;
				}

				cineware::Int32 ReflectionLayerBaseId = ReflectionLayer->GetDataID();
				cineware::Int32 ReflectionLayerFlags = ReflectionLayer->GetFlags();

				// Don't fetch colors from reflectance layers that, regardless of fresnel function, don't seem to contribute a lot to main base color
				int32 LayerType = MelangeGetInt32(InC4DMaterialPtr, ReflectionLayerBaseId+REFLECTION_LAYER_MAIN_DISTRIBUTION);
				if (LayerType == REFLECTION_DISTRIBUTION_SPECULAR_PHONG || LayerType == REFLECTION_DISTRIBUTION_SPECULAR_BLINN || LayerType == REFLECTION_DISTRIBUTION_IRAWAN)
				{
					continue;
				}

				// Whether the layer is marked as visible (eye icon left of layer name)
				if (ReflectionLayerFlags & REFLECTION_FLAG_ACTIVE)
				{
					// Dropdown for Normal/Add to the right of layer name
					int32 BlendMode = MelangeGetInt32(InC4DMaterialPtr, ReflectionLayerBaseId+REFLECTION_LAYER_MAIN_BLEND_MODE);

					// Slider/percentage value describing the layer opacity, to the right of Normal/Add dropdown
					float Opacity = MelangeGetFloat(InC4DMaterialPtr, ReflectionLayerBaseId+REFLECTION_LAYER_MAIN_OPACITY);

					bUseReflectionColor = true;
					FVector LayerColor = MelangeGetLayerColor(InC4DMaterialPtr, ReflectionLayerBaseId + REFLECTION_LAYER_COLOR_COLOR,
															  ReflectionLayerBaseId + REFLECTION_LAYER_COLOR_BRIGHTNESS);

					// This is a temporary solution in order to let some color from reflectance layers factor in to the final basecolor depending on their
					// fresnel function
					cineware::Int32 FresnelMode = InC4DMaterialPtr->GetParameter(ReflectionLayerBaseId + REFLECTION_LAYER_FRESNEL_MODE, C4DData) ? C4DData.GetInt32() : REFLECTION_FRESNEL_NONE;
					switch (FresnelMode)
					{
					case REFLECTION_FRESNEL_NONE:
						Opacity *= 1.0f;  // The reflectance layer looks like a solid, opaque layer
						break;
					case REFLECTION_FRESNEL_DIELECTRIC:
						Opacity *= 0.0f;  // The reflectance layer is used mostly for highlights and specular reflections
						break;
					case REFLECTION_FRESNEL_CONDUCTOR:
						Opacity *= 0.4f;  // The reflectance layer looks like a transparent coat or overlay
						break;
					default:
						break;
					}

					// Normal
					if (BlendMode == 0)
					{
						ReflectionColor = LayerColor * Opacity + ReflectionColor * (1 - Opacity);
					}
					// Add
					else if(BlendMode == 1)
					{
						ReflectionColor = LayerColor * Opacity + ReflectionColor;
					}
				}
			}

			AddBoolToMaterial(MaterialPtr, TEXT("Use_ReflectionColor"), bUseReflectionColor);
			if (bUseReflectionColor)
			{
				// Global Reflection Brightness and Specular Brightness on Layers tab
				float GlobalReflection = static_cast<float>(MelangeGetDouble(InC4DMaterialPtr, REFLECTION_LAYER_GLOBAL_REFLECTION));
				float GlobalSpecular = static_cast<float>(MelangeGetDouble(InC4DMaterialPtr, REFLECTION_LAYER_GLOBAL_SPECULAR));

				// Approximation of the combined effect of those. This doesn't make much sense
				// as these are different effects and applied differently, but this is all a
				// temp solution until we get proper material graphs
				float ReflectionChannelColorWeight = (GlobalReflection * 0.75f + GlobalSpecular * 0.25f);
				AddFloatToMaterial(MaterialPtr, TEXT("ReflectionColor_Strength"), ReflectionChannelColorWeight);
				AddColorToMaterial(MaterialPtr, TEXT("ReflectionColor"), FLinearColor(ReflectionColor));
			}

			// Only set those one for the last Layer of reflection
			cineware::ReflectionLayer* ReflectionLayer = InC4DMaterialPtr->GetReflectionLayerIndex(0);

			bool bUseReflectance = (ReflectionLayer != nullptr);
			AddBoolToMaterial(MaterialPtr, TEXT("Use_Reflectance"), bUseReflectance);
			if (bUseReflectance)
			{
				cineware::Int32 ReflectionLayerBaseId = ReflectionLayer->GetDataID();

				float SpecularStrength = MelangeGetFloat(InC4DMaterialPtr, ReflectionLayerBaseId + REFLECTION_LAYER_MAIN_VALUE_SPECULAR);
				AddFloatToMaterial(MaterialPtr, TEXT("Specular_Strength"), SpecularStrength);

				cineware::BaseList2D* RoughnessShader = MelangeGetLink(InC4DMaterialPtr, ReflectionLayerBaseId + REFLECTION_LAYER_MAIN_SHADER_ROUGHNESS);
				FString RoughnessMapPath = GetBaseShaderTextureFilePath(RoughnessShader);
				TSharedPtr<IDatasmithTextureElement> RoughnessMap1 = ImportTexture(RoughnessMapPath, EDatasmithTextureMode::Diffuse);
				AddTextureToMaterial(MaterialPtr, TEXT("RoughnessMap1"), RoughnessMap1);

				bool bUseRoughnessMap = !RoughnessMapPath.IsEmpty();
				AddBoolToMaterial(MaterialPtr, TEXT("Use_RoughnessMap"), bUseRoughnessMap);
				if (bUseRoughnessMap)
				{
					float RoughnessMapStrength = MelangeGetFloat(InC4DMaterialPtr, ReflectionLayerBaseId + REFLECTION_LAYER_MAIN_VALUE_ROUGHNESS);
					AddFloatToMaterial(MaterialPtr, TEXT("RoughnessMap1_Strength"), RoughnessMapStrength);
				}
				else
				{
					float RoughnessStrength = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_SPECULAR_WIDTH /*appears to be the computed roughness*/);
					AddFloatToMaterial(MaterialPtr, TEXT("Roughness_Strength"), RoughnessStrength);
				}

				int32 FresnelMode = MelangeGetInt32(InC4DMaterialPtr, ReflectionLayerBaseId + REFLECTION_LAYER_FRESNEL_MODE);

				bool bUseMetalic = (FresnelMode == REFLECTION_FRESNEL_CONDUCTOR);
				AddBoolToMaterial(MaterialPtr, TEXT("Use_Metalic"), bUseMetalic);
				if (bUseMetalic)
				{
					AddFloatToMaterial(MaterialPtr, TEXT("Metalic_Amount"), 0.5f);

					cineware::BaseList2D* MetallicShader = MelangeGetLink(InC4DMaterialPtr, ReflectionLayerBaseId + REFLECTION_LAYER_TRANS_TEXTURE);
					FString MetallicMapPath = GetBaseShaderTextureFilePath(MetallicShader);
					TSharedPtr<IDatasmithTextureElement> MetalicMap = ImportTexture(MetallicMapPath, EDatasmithTextureMode::Specular);
					AddTextureToMaterial(MaterialPtr, TEXT("MetalicMap"), MetalicMap);

					bool bUseMetalicMap = !MetallicMapPath.IsEmpty();
					AddBoolToMaterial(MaterialPtr, TEXT("Use_MetalicMap"), bUseMetalicMap);
				}
			}
		}
	}

	// AO
	bool bUseAO = InC4DMaterialPtr->GetChannelState(CHANNEL_DIFFUSION);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_AO"), bUseAO);
	if (bUseAO)
	{
		cineware::BaseList2D* DiffusionShader = MelangeGetLink(InC4DMaterialPtr, cineware::MATERIAL_DIFFUSION_SHADER);
		FString AOMapPath = GetBaseShaderTextureFilePath(DiffusionShader);
		TSharedPtr<IDatasmithTextureElement> AOMap = ImportTexture(AOMapPath, EDatasmithTextureMode::Diffuse);
		AddTextureToMaterial(MaterialPtr, TEXT("AO_Map"), AOMap);

		if (!AOMapPath.IsEmpty())
		{
			float AOStrength = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_DIFFUSION_TEXTURESTRENGTH);
			AddFloatToMaterial(MaterialPtr, TEXT("AO_Strength"), AOStrength);
		}
	}

	// Alpha
	bool bUseAlpha = !bUseTransparency && InC4DMaterialPtr->GetChannelState(CHANNEL_ALPHA);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_Alpha"), bUseAlpha);
	if (bUseAlpha)
	{
		MaterialPtr->SetMaterialType(EDatasmithReferenceMaterialType::CutOut);

		cineware::BaseList2D* AlphaShader = MelangeGetLink(InC4DMaterialPtr, cineware::MATERIAL_ALPHA_SHADER);
		FString AlphaMapPath = GetBaseShaderTextureFilePath(AlphaShader);
		TSharedPtr<IDatasmithTextureElement> AlphaMap = ImportTexture(AlphaMapPath, EDatasmithTextureMode::Diffuse);
		AddTextureToMaterial(MaterialPtr, TEXT("Alpha_Map"), AlphaMap);

		bool bUseAlphaInvert = MelangeGetBool(InC4DMaterialPtr, cineware::MATERIAL_ALPHA_INVERT);
		AddBoolToMaterial(MaterialPtr, TEXT("Use_Alpha_Invert"), bUseAlphaInvert);
	}

	// Normal
	bool bUseNormal = InC4DMaterialPtr->GetChannelState(CHANNEL_NORMAL);
	AddBoolToMaterial(MaterialPtr, TEXT("Use_Normal"), bUseNormal);
	if (bUseNormal)
	{
		cineware::BaseList2D* NormalShader = MelangeGetLink(InC4DMaterialPtr, cineware::MATERIAL_NORMAL_SHADER);
		FString NormalMapPath = GetBaseShaderTextureFilePath(NormalShader);
		TSharedPtr<IDatasmithTextureElement> NormalMap = ImportTexture(NormalMapPath, EDatasmithTextureMode::Normal);
		AddTextureToMaterial(MaterialPtr, TEXT("Normal_Map"), NormalMap);

		if (!NormalMapPath.IsEmpty())
		{
			float NormalMapStrength = MelangeGetFloat(InC4DMaterialPtr, cineware::MATERIAL_NORMAL_STRENGTH);
			AddFloatToMaterial(MaterialPtr, TEXT("Normal_Strength"), NormalMapStrength);
		}
	}

	DatasmithScene->AddMaterial(MaterialPtr);
	return MaterialPtr;
}

TSharedPtr<IDatasmithMaterialInstanceElement> FDatasmithC4DImporter::ImportSimpleColorMaterial(cineware::BaseObject* Object, int32 UseColor)
{
	TOptional<FString> DatasmithName = GetMelangeBaseList2dID(Object);
	if (!DatasmithName)
	{
		return TSharedPtr<IDatasmithMaterialInstanceElement>();
	}
	FString DatasmithLabel = FDatasmithUtils::SanitizeObjectName(MelangeObjectName(Object) + TEXT("_DisplayColor"));

	FVector DisplayColor{1.0f, 1.0f, 1.0f};
	if (UseColor == cineware::ID_BASEOBJECT_USECOLOR_AUTOMATIC || UseColor == cineware::ID_BASEOBJECT_USECOLOR_ALWAYS)
	{
		DisplayColor = MelangeGetColor(Object, cineware::ID_BASEOBJECT_COLOR);
	}
	else if (UseColor == cineware::ID_BASEOBJECT_USECOLOR_LAYER)
	{
		if (cineware::LayerObject* LayerObject = static_cast<cineware::LayerObject*>(MelangeGetLink(Object, cineware::ID_LAYER_LINK)))
		{
			DisplayColor = MelangeGetColor(LayerObject, cineware::ID_LAYER_COLOR);
		}
		else
		{
			DisplayColor = GetDocumentDefaultColor();
		}
	}

	FString MaterialHash = TEXT("DisplayColor_") + LexToString(GetTypeHash(DisplayColor));

	TSharedPtr<IDatasmithMaterialInstanceElement>& Material = MaterialNameToMaterialElement.FindOrAdd(MaterialHash);
	if (Material.IsValid())
	{
		return Material;
	}

	Material = FDatasmithSceneFactory::CreateMaterialInstance(*(DatasmithName.GetValue()));
	Material->SetLabel(*DatasmithLabel);
	Material->SetMaterialType(EDatasmithReferenceMaterialType::Opaque);

	// Color
	AddColorToMaterial(Material, TEXT("Color"), FLinearColor(DisplayColor));
	AddBoolToMaterial(Material, TEXT("Use_Color"), true);
	AddBoolToMaterial(Material, TEXT("Use_ColorMap"), false);

	DatasmithScene->AddMaterial(Material);
	return Material;
}

bool FDatasmithC4DImporter::ImportMaterialHierarchy(cineware::BaseMaterial* InC4DMaterialPtr)
{
	// Reinitialize the scene material map and texture set.
	MaterialNameToMaterialElement.Empty();

	for (; InC4DMaterialPtr; InC4DMaterialPtr = InC4DMaterialPtr->GetNext())
	{
		if (InC4DMaterialPtr->GetType() == Mmaterial)
		{
			if (TSharedPtr<IDatasmithMaterialInstanceElement> DatasmithMaterial = ImportMaterial(static_cast<cineware::Material*>(InC4DMaterialPtr)))
			{
				MaterialNameToMaterialElement.Add(DatasmithMaterial->GetName(), DatasmithMaterial);
			}
			else
			{
				return false;
			}
		}
	}

	return true;
}

FString FDatasmithC4DImporter::CustomizeMaterial(const FString& InMaterialID, const FString& InMeshID, cineware::TextureTag* InTextureTag)
{
	FString CustomMaterialID = InMaterialID + InMeshID;

	if (MaterialNameToMaterialElement.Contains(CustomMaterialID))
	{
		return CustomMaterialID;
	}

	if (MaterialNameToMaterialElement.Contains(InMaterialID))
	{
		cineware::GeData Data;
		float OffsetX = MelangeGetFloat(InTextureTag, cineware::TEXTURETAG_OFFSETX);
		float OffsetY = MelangeGetFloat(InTextureTag, cineware::TEXTURETAG_OFFSETY);
		float TilesX = MelangeGetFloat(InTextureTag, cineware::TEXTURETAG_TILESX);
		float TilesY = MelangeGetFloat(InTextureTag, cineware::TEXTURETAG_TILESY);

		if (OffsetX != 0.0F || OffsetY != 0.0F || TilesX != 1.0F || TilesY != 1.0F)
		{
			TSharedPtr<IDatasmithMaterialInstanceElement> CustomizedMaterial = FDatasmithSceneFactory::CreateMaterialInstance(*CustomMaterialID);

			// Create a copy of the original material
			TSharedPtr<IDatasmithMaterialInstanceElement> OriginalMaterial = MaterialNameToMaterialElement[InMaterialID];
			for (int32 PropertyIndex = 0; PropertyIndex < OriginalMaterial->GetPropertiesCount(); ++PropertyIndex)
			{
				CustomizedMaterial->AddProperty(OriginalMaterial->GetProperty(PropertyIndex));
			}
			CustomizedMaterial->SetLabel(OriginalMaterial->GetLabel());

			AddFloatToMaterial(CustomizedMaterial, TEXT("Offset_U"), OffsetX);
			AddFloatToMaterial(CustomizedMaterial, TEXT("Offset_V"), OffsetY);
			AddFloatToMaterial(CustomizedMaterial, TEXT("Tile_U"), TilesX);
			AddFloatToMaterial(CustomizedMaterial, TEXT("Tile_V"), TilesY);

			MaterialNameToMaterialElement.Add(CustomMaterialID, CustomizedMaterial);

			DatasmithScene->AddMaterial(CustomizedMaterial);
			return CustomMaterialID;
		}
	}

	return InMaterialID;
}

TMap<int32, FString> FDatasmithC4DImporter::GetCustomizedMaterialAssignment(const FString& DatasmithMeshName, const TArray<cineware::TextureTag*>& TextureTags)
{
	TMap<int32, FString> SlotToMaterialName;

	// Create customized materials for all the used texture tags. This because each tag
	// actually represents a material "instance", and might have different settings like texture tiling
	for (int32 SlotIndex = 0; SlotIndex < TextureTags.Num(); ++SlotIndex)
	{
		cineware::TextureTag* Tag = TextureTags[SlotIndex];

		FString CustomizedMaterialName;
		cineware::BaseList2D* TextureMaterial = Tag ? MelangeGetLink(Tag, cineware::TEXTURETAG_MATERIAL) : nullptr;
		if (TextureMaterial)
		{
			// This can also return an existing material without necessarily spawning a new instance
			if (TOptional<FString> MaterialID = GetMelangeBaseList2dID(TextureMaterial))
			{
				CustomizedMaterialName = CustomizeMaterial(MaterialID.GetValue(), DatasmithMeshName, Tag);
			}
		}

		SlotToMaterialName.Add(SlotIndex, CustomizedMaterialName);
	}

	return SlotToMaterialName;
}

TSharedPtr<IDatasmithMeshActorElement> FDatasmithC4DImporter::ImportPolygon(cineware::PolygonObject* PolyObject, const FString& DatasmithActorName, const FString& DatasmithActorLabel, const TArray<cineware::TextureTag*>& TextureTags)
{
	FMD5Hash PolygonHash = ComputePolygonDataHash(PolyObject);
	FString HashString = BytesToHex(PolygonHash.GetBytes(), PolygonHash.GetSize());

	TOptional<FString> MelangeID = MelangeObjectID(PolyObject);
	if (!MelangeID.IsSet())
	{
		return nullptr;
	}
	const FString& DatasmithMeshName = MelangeID.GetValue();

	IDatasmithMeshElement* ResultMeshElement = nullptr;
	if (TSharedRef<IDatasmithMeshElement>* PreviousMesh = PolygonHashToMeshElement.Find(HashString))
	{
		ResultMeshElement = &PreviousMesh->Get();
	}
	else
	{
		TSharedPtr<IDatasmithMeshElement> MeshElement = ImportMesh(PolyObject, *DatasmithMeshName, DatasmithActorLabel);
		ResultMeshElement = MeshElement.Get();

		// Set the polygon hash as the file hash. It will be checked by Datasmith in
		// FDatasmithImporter::FilterElementsToImport to know if a mesh has changed and
		// the asset needs to be replaced during reimport
		ResultMeshElement->SetFileHash(PolygonHash);

		PolygonHashToMeshElement.Add(HashString, MeshElement.ToSharedRef());
	}

	TSharedPtr<IDatasmithMeshActorElement> MeshActorElement = FDatasmithSceneFactory::CreateMeshActor(*DatasmithActorName);
	MeshActorElement->SetLabel(*DatasmithActorLabel);
	MeshActorElement->SetStaticMeshPathName(ResultMeshElement->GetName());

	// Check if we still need to assign materials to the base mesh
	bool bMeshHasMaterialAssignments = false;
	for ( int32 SlotIndex = 0; SlotIndex < ResultMeshElement->GetMaterialSlotCount(); ++SlotIndex )
	{
		TSharedPtr<IDatasmithMaterialIDElement> SlotIDElement = ResultMeshElement->GetMaterialSlotAt(SlotIndex);
		if ( !SlotIDElement.IsValid() )
		{
			continue;
		}

		if ( ResultMeshElement->GetMaterial( SlotIDElement->GetId() ) != nullptr )
		{
			bMeshHasMaterialAssignments = true;
			break;
		}
	}

	const int32 UseColor = MelangeGetInt32(PolyObject, cineware::ID_BASEOBJECT_USECOLOR);

	// Add material overrides
	TMap<int32, FString> SlotIndexToMaterialName = GetCustomizedMaterialAssignment(DatasmithMeshName, TextureTags);
	for (const TPair<int32, FString>& Pair : SlotIndexToMaterialName)
	{
		int32 SlotIndex = Pair.Key;
		FString MaterialName = Pair.Value;

		// Pick whether we use the display color material or a texturetag material
		TSharedPtr<IDatasmithMaterialInstanceElement> TargetMaterial = nullptr;
		if (UseColor == cineware::ID_BASEOBJECT_USECOLOR_ALWAYS || UseColor == cineware::ID_BASEOBJECT_USECOLOR_LAYER)
		{
			TargetMaterial = ImportSimpleColorMaterial(PolyObject, UseColor);
		}
		else if (UseColor == cineware::ID_BASEOBJECT_USECOLOR_AUTOMATIC)
		{
			if (MaterialName.IsEmpty())
			{
				TargetMaterial = ImportSimpleColorMaterial(PolyObject, UseColor);
			}
			else
			{
				TSharedPtr<IDatasmithMaterialInstanceElement>* FoundMaterial = MaterialNameToMaterialElement.Find(MaterialName);
				if (FoundMaterial && FoundMaterial->IsValid())
				{
					TargetMaterial = *FoundMaterial;
				}
			}
		}

		// Valid material, set it as override
		if (TargetMaterial.IsValid())
		{
			MaterialName = TargetMaterial->GetName();
		}

		// We must always create and set material overrides for all found materials, as a PolygonObject imported later may
		// cause the base mesh material to be reset to unassigned (below)
		TSharedRef<IDatasmithMaterialIDElement> MaterialIDElement = FDatasmithSceneFactory::CreateMaterialId(*MaterialName);
		MaterialIDElement->SetId(SlotIndex);
		MeshActorElement->AddMaterialOverride(MaterialIDElement);

		// If we have an unassigned material, we *must* set it on the base mesh, as we can't create a material overrides to "clear" a material slot
		if (!bMeshHasMaterialAssignments || MaterialName.IsEmpty())
		{
			ResultMeshElement->SetMaterial(*MaterialName, SlotIndex);
		}
	}

	return MeshActorElement;
}

void MarkActorsAsParticlesRecursive(cineware::BaseObject* ActorObject, TSet<cineware::BaseObject*>& ParticleActors)
{
	if (ActorObject == nullptr)
	{
		return;
	}

	ParticleActors.Add(ActorObject);

	MarkActorsAsParticlesRecursive(ActorObject->GetDown(), ParticleActors);
	MarkActorsAsParticlesRecursive(ActorObject->GetNext(), ParticleActors);
}

void FDatasmithC4DImporter::MarkActorsAsParticles(cineware::BaseObject* EmitterObject, cineware::BaseObject* EmittersCache)
{
	if (EmitterObject == nullptr || EmittersCache == nullptr)
	{
		return;
	}

	// C4D only emits mesh "particles" if this "Show Objects" checkbox is checked. Else it just emits actual particles
	cineware::GeData Data;
	if (EmitterObject->GetParameter(cineware::PARTICLEOBJECT_SHOWOBJECTS, Data) && Data.GetType() == cineware::DA_LONG && Data.GetBool())
	{
		MarkActorsAsParticlesRecursive(EmittersCache->GetDown(), ParticleActors);
	}
}

namespace
{
	cineware::Float MelangeFPS = 0;

	void AddFrameValueToAnimMap(
		cineware::BaseObject* Object,
		int32 FrameNumber,
		int32 TransformVectorIndex,
		EDatasmithTransformType TransformType,
		cineware::Float FrameValue,
		cineware::Int32 MelangeTransformType,
		FVector& InitialSize,
		TMap<int32, TMap<EDatasmithTransformType, FVector>>& TransformFrames,
		TMap<EDatasmithTransformType, FVector> InitialValues
	)
	{
		TMap<EDatasmithTransformType, FVector>* FrameValues = TransformFrames.Find(FrameNumber);
		if (!FrameValues)
		{
			FrameValues = &TransformFrames.Add(FrameNumber);
		}
		FVector* TransformValues = FrameValues->Find(TransformType);
		if (!TransformValues)
		{
			TransformValues = &FrameValues->Add(TransformType);
			*TransformValues = *InitialValues.Find(TransformType);
		}
		float Value = static_cast<float>(FrameValue);
		if (TransformType == EDatasmithTransformType::Scale)
		{
			if (MelangeTransformType == 1100) //Size
			{
				//Value is the absolute size, so First key = scaling of 1.0
				if (InitialSize[TransformVectorIndex] == 0)
				{
					InitialSize[TransformVectorIndex] = Value;
					Value = 1.0;
				}
				else
				{
					Value /= (float)InitialSize[TransformVectorIndex];		// LWC_TODO: InitialSize may be doubles!
				}
			}
		}
		(*TransformValues)[TransformVectorIndex] = Value;
	}
}

void FDatasmithC4DImporter::ImportAnimations(TSharedPtr<IDatasmithActorElement> ActorElement)
{
	cineware::BaseObject* Object = *ActorElementToAnimationSources.Find(ActorElement.Get());
	cineware::Int32 ObjectType = Object->GetType();

	TMap<EDatasmithTransformType, FVector> InitialValues;
	cineware::Vector MelangeRotation = Object->GetRelRot();
	InitialValues.Add(EDatasmithTransformType::Rotation) = FVector(static_cast<float>(MelangeRotation.x), static_cast<float>(MelangeRotation.y), static_cast<float>(MelangeRotation.z));
	InitialValues.Add(EDatasmithTransformType::Translation) = MelangeVectorToFVector(Object->GetRelPos());
	InitialValues.Add(EDatasmithTransformType::Scale) = MelangeVectorToFVector(Object->GetRelScale());

	TMap<int32, TMap<EDatasmithTransformType, FVector>> TransformFrames;
	FVector InitialSize(0, 0, 0);

	// If we have AlignToSpline animations, the splines are stored with their points in world space,
	// so we must move them into the object's local space
	cineware::Matrix WorldToLocal = ~Object->GetUpMg();

	cineware::ROTATIONORDER RotationOrder = Object->GetRotationOrder();

	// Import animations on the object's tags
	for(cineware::BaseTag* Tag = Object->GetFirstTag(); Tag; Tag=Tag->GetNext())
	{
		cineware::Int32 TagType = Tag->GetType();

		if (TagType == Tcrane && ObjectType == Ocamera)
		{
			TSharedRef<FCraneCameraAttributes>* FoundAttributes = CraneCameraToAttributes.Find(Object);
			if (!FoundAttributes)
			{
				UE_LOGF(LogDatasmithC4DImport, Warning, "Trying to parse animations for crane camera '%ls', but it doesn't have crane camera attributes!", *MelangeObjectName(Object));
				continue;
			}

			TMap<int32, cineware::CCurve*> CurvesByAttribute;

			cineware::BaseTime MinStartTime(FLT_MAX);
			cineware::BaseTime MaxEndTime(-FLT_MAX);

			// Get tracks for all animated properties
			for (cineware::CTrack* Track = Tag->GetFirstCTrack(); Track; Track = Track->GetNext())
			{
				cineware::DescID TrackDescID = Track->GetDescriptionID();
				int32 Depth = TrackDescID.GetDepth();
				if (Depth != 1)
				{
					continue;
				}
				int32 AttributeID = TrackDescID[0].id;

				cineware::CCurve* Curve = Track->GetCurve();
				if (!Curve || Curve->GetKeyCount() == 0)
				{
					continue;
				}

				MinStartTime = FMath::Min(MinStartTime, Curve->GetStartTime());
				MaxEndTime = FMath::Max(MaxEndTime, Curve->GetEndTime());

				CurvesByAttribute.Add(AttributeID, Curve);
			}

			// Bake every frame
			// We could get just the frames where at least one attribute has been keyed, but
			// the default is to have a sigmoid interpolation anyway, which means that the final
			// transform will almost always need to be baked frame-by-frame. We might as well
			// keep things simple
			int32 FirstFrame = MinStartTime.GetFrame(MelangeFPS);
			int32 LastFrame = MaxEndTime.GetFrame(MelangeFPS);
			for (int32 FrameNumber = FirstFrame; FrameNumber <= LastFrame; ++FrameNumber)
			{
				cineware::BaseTime FrameTime = cineware::BaseTime(MinStartTime.Get() + cineware::Float((FrameNumber-FirstFrame) * (1.0 / MelangeFPS)));

				// Construct the FCraneCameraAttributes struct for this frame
				FCraneCameraAttributes AttributesForFrame = **FoundAttributes;
				for (const TPair<int, cineware::CCurve*>& Pair : CurvesByAttribute)
				{
					const cineware::CCurve* AttributeCurve = Pair.Value;
					const int32& AttributeID = Pair.Key;
					double AttributeValue = AttributeCurve->GetValue(FrameTime);

					AttributesForFrame.SetAttributeByID(AttributeID, AttributeValue);
				}

				// Note: bCompensatePitch and bCompensateHeading will also be fetched but as of SDK version 20.0_259890
				// the actual CCurve and tag attribute seem to always have false for them, regardless of whether these
				// options are checked or not in C4D. So we restore them to what is the frame-zero value for this camera,
				// which can be fetched correctly
				AttributesForFrame.bCompensatePitch = (*FoundAttributes)->bCompensatePitch;
				AttributesForFrame.bCompensateHeading = (*FoundAttributes)->bCompensateHeading;

				FTransform TransformForFrame = CalculateCraneCameraTransform(AttributesForFrame);
				FVector Translation = TransformForFrame.GetTranslation();
				FVector RotationEuler = TransformForFrame.GetRotation().Euler();

				for (int32 Component = 0; Component < 3; ++Component)
				{
					AddFrameValueToAnimMap(
						Object,
						FrameNumber,
						Component,
						EDatasmithTransformType::Translation,
						Translation[Component],
						cineware::ID_BASEOBJECT_REL_POSITION,
						InitialSize,
						TransformFrames,
						InitialValues);

					AddFrameValueToAnimMap(
						Object,
						FrameNumber,
						Component,
						EDatasmithTransformType::Rotation,
						FMath::DegreesToRadians(RotationEuler[Component]),
						cineware::ID_BASEOBJECT_REL_ROTATION,
						InitialSize,
						TransformFrames,
						InitialValues);
				}
			}
		}
		// TODO: CraneCameras can also have an AlignToSpline tag, so that the crane camera
		// base moves along the spline. We don't support that for now
		else if (TagType == Taligntospline)
		{
			cineware::SplineObject* SplineObj = static_cast<cineware::SplineObject*>(MelangeGetLink(Tag, cineware::ALIGNTOSPLINETAG_LINK));
			if (!SplineObj)
			{
				continue;
			}

			TArray<FRichCurve>* FoundSpline = SplineCurves.Find(SplineObj);
			if (!FoundSpline)
			{
				UE_LOGF(LogDatasmithC4DImport, Error, "Did not find target spline object '%ls' for %ls's AlignToSpline animation!", *MelangeObjectName(SplineObj), *MelangeObjectName(Object));
				continue;
			}

			for (cineware::CTrack* Track = Tag->GetFirstCTrack(); Track; Track=Track->GetNext())
			{
				cineware::DescID TrackDescID = Track->GetDescriptionID();

				int32 Depth = TrackDescID.GetDepth();
				if (Depth != 1)
				{
					continue;
				}

				cineware::Int32 MelangeTransformType = TrackDescID[0].id;
				if (MelangeTransformType != cineware::ALIGNTOSPLINETAG_POSITION)
				{
					continue;
				}

				cineware::CCurve* Curve = Track->GetCurve();
				if (!Curve)
				{
					continue;
				}

				// We need to bake every keyframe, as we need to eval the richcurves for the spline position
				cineware::BaseTime StartTime = Curve->GetStartTime();
				cineware::BaseTime EndTime = Curve->GetEndTime();
				int32 FirstFrame = StartTime.GetFrame(MelangeFPS);
				int32 LastFrame = EndTime.GetFrame(MelangeFPS);
				for (int32 FrameNumber = FirstFrame; FrameNumber <= LastFrame; ++FrameNumber)
				{
					// Uses the timing curve to find the percentage of the spline path at which we must sample
					// (e.g. 0.0 -> start; 0.5 -> middle; 1.0 -> end)
					float Percent = (float)Curve->GetValue(cineware::BaseTime(StartTime.Get() + cineware::Float((FrameNumber-FirstFrame) * (1.0 / MelangeFPS))));

					// Target spline point in our local space
					cineware::Vector Location = WorldToLocal * cineware::Vector((*FoundSpline)[0].Eval(Percent),
																			  (*FoundSpline)[1].Eval(Percent),
																			  (*FoundSpline)[2].Eval(Percent));
					for (int32 Component = 0; Component < 3; ++Component)
					{
						float ComponentValue = (float)Location[Component];
						AddFrameValueToAnimMap(
							Object,
							FrameNumber,
							Component,
							EDatasmithTransformType::Translation,
							ComponentValue,
							cineware::ID_BASEOBJECT_REL_POSITION,
							InitialSize,
							TransformFrames,
							InitialValues);
					}
				}
			}
		}
	}

	// Get the last point in time where we have a valid key
	cineware::BaseTime MaxTime(-1.0);
	for(cineware::CTrack* Track = Object->GetFirstCTrack(); Track; Track=Track->GetNext())
	{
		cineware::DescID TrackDescID = Track->GetDescriptionID();
		if (TrackDescID.GetDepth() != 2)
		{
			continue;
		}

		if (TrackDescID[1].id != cineware::VECTOR_X && TrackDescID[1].id != cineware::VECTOR_Y && TrackDescID[1].id != cineware::VECTOR_Z)
		{
			continue;
		}

		if (cineware::CCurve* Curve = Track->GetCurve())
		{
			MaxTime = FMath::Max(MaxTime, Curve->GetEndTime());
		}
	}

	// Import animations on the object's attributes
	for(cineware::CTrack* Track = Object->GetFirstCTrack(); Track; Track=Track->GetNext())
	{
		cineware::DescID TrackDescID = Track->GetDescriptionID();
		if (TrackDescID.GetDepth() != 2)
		{
			continue;
		}

		int32 TransformVectorIndex;
		switch (TrackDescID[1].id)
		{
		case cineware::VECTOR_X: TransformVectorIndex = 0; break;
		case cineware::VECTOR_Y: TransformVectorIndex = 1; break;
		case cineware::VECTOR_Z: TransformVectorIndex = 2; break;
		default: continue;
		}

		EDatasmithTransformType TransformType;
		cineware::Int32 MelangeTransformType = TrackDescID[0].id;
		switch (MelangeTransformType)
		{
		case cineware::ID_BASEOBJECT_REL_POSITION: TransformType = EDatasmithTransformType::Translation; break;
		case cineware::ID_BASEOBJECT_REL_ROTATION: TransformType = EDatasmithTransformType::Rotation; break;
		case 1100: /*size*/
		case cineware::ID_BASEOBJECT_REL_SCALE:	  TransformType = EDatasmithTransformType::Scale; break;
		default: continue;
		}

		cineware::CCurve* Curve = Track->GetCurve();
		if (!Curve)
		{
			continue;
		}

		for (cineware::Int32 KeyIndex = 0; KeyIndex < Curve->GetKeyCount(); KeyIndex++)
		{
			cineware::CKey* CurrentKey = Curve->GetKey(KeyIndex);
			cineware::CINTERPOLATION Interpolation = CurrentKey->GetInterpolation();

			int32 FrameNumber = CurrentKey->GetTime().GetFrame(MelangeFPS);
			cineware::Float FrameValue = CurrentKey->GetValue();
			AddFrameValueToAnimMap(
				Object,
				FrameNumber,
				TransformVectorIndex,
				TransformType,
				FrameValue,
				MelangeTransformType,
				InitialSize,
				TransformFrames,
				InitialValues);

			if (KeyIndex < Curve->GetKeyCount() - 1)
			{
				//"Bake" the animation by generating a key for each frame between this Key and the next one
				cineware::CKey* NextKey = Curve->GetKey(KeyIndex+1);
				cineware::BaseTime CurrentKeyTime = CurrentKey->GetTime();
				cineware::BaseTime NextKeyTime = NextKey->GetTime();
				int32 NextKeyFrameNumber = NextKeyTime.GetFrame(MelangeFPS);
				int32 FrameCount = NextKeyFrameNumber - FrameNumber;
				cineware::Float ElapsedTime = NextKeyTime.Get() - CurrentKeyTime.Get();
				for (int32 FrameIndex = 1; FrameIndex < FrameCount; FrameIndex++)
				{
					FrameNumber++;
					FrameValue = Curve->GetValue(cineware::BaseTime(CurrentKeyTime.Get() + ((ElapsedTime / FrameCount) * FrameIndex)));
					AddFrameValueToAnimMap(
						Object,
						FrameNumber,
						TransformVectorIndex,
						TransformType,
						FrameValue,
						MelangeTransformType,
						InitialSize,
						TransformFrames,
						InitialValues);
				}
			}
		}

		// Make sure the transform frame values remain at their last valid value up until the end of the
		// animation. We use FVectors to store all three components at once, if we don't do this we will
		// incorrectly think that components whose animation curves end early have gone back to zero
		cineware::Float LastValue = Curve->GetValue(Curve->GetEndTime());
		cineware::Int32 FirstFrameToFill = Curve->GetEndTime().GetFrame(MelangeFPS) + 1;
		cineware::Int32 LastFrameToFill = MaxTime.GetFrame(MelangeFPS);
		for (cineware::Int32 Frame = FirstFrameToFill; Frame <= LastFrameToFill; ++Frame)
		{
			AddFrameValueToAnimMap(
				Object,
				Frame,
				TransformVectorIndex,
				TransformType,
				LastValue,
				MelangeTransformType,
				InitialSize,
				TransformFrames,
				InitialValues);
		}
	}

	// No tags or object attribute animations
	if (TransformFrames.Num() == 0)
	{
		return;
	}

	// Prevent actor from being optimized away
	NamesOfActorsToKeep.Add(ActorElement->GetName());

	// Add a visibility track to simulate the particle spawning and despawning, if this is a particle actor.
	// It seems like the particles have keys where they are visible: Before the first key the particles haven't
	// spawned yet, and after the last key the particles disappear.
	if (ParticleActors.Contains(Object))
	{
		int32 FirstFrameAdded = MAX_int32;
		int32 LastFrameAdded = -1;
		for (const TPair<int32, TMap<EDatasmithTransformType, FVector>>& Pair : TransformFrames)
		{
			LastFrameAdded = FMath::Max(LastFrameAdded, Pair.Key);
			FirstFrameAdded = FMath::Min(FirstFrameAdded, Pair.Key);
		}

		TSharedRef<IDatasmithVisibilityAnimationElement> VisibilityAnimation = FDatasmithSceneFactory::CreateVisibilityAnimation(ActorElement->GetName());

		// Before our first frame we should be invisible
		if (FirstFrameAdded != 0)
		{
			VisibilityAnimation->AddFrame(FDatasmithVisibilityFrameInfo(0, false));
		}

		// We're always visible during our animation
		VisibilityAnimation->AddFrame(FDatasmithVisibilityFrameInfo(FirstFrameAdded, true));
		VisibilityAnimation->AddFrame(FDatasmithVisibilityFrameInfo(LastFrameAdded, true));

		// After our last frame we should be visible, but don't add a new key if that is also the last frame of the document
		cineware::GeData Data;
		if (C4dDocument->GetParameter(cineware::DOCUMENT_MAXTIME, Data) && Data.GetType() == cineware::DA_TIME)
		{
			const cineware::BaseTime& Time = Data.GetTime();
			int32 LastDocumentFrame = Time.GetFrame(MelangeFPS);

			if (LastFrameAdded < LastDocumentFrame)
			{
				VisibilityAnimation->AddFrame(FDatasmithVisibilityFrameInfo(LastFrameAdded+1, false));
			}
		}

		LevelSequence->AddAnimation(VisibilityAnimation);
	}

	TSharedRef< IDatasmithTransformAnimationElement > Animation = FDatasmithSceneFactory::CreateTransformAnimation(ActorElement->GetName());
	for (int32 TransformTypeIndex = 0; TransformTypeIndex < 3; TransformTypeIndex++)
	{
		EDatasmithTransformType TransFormType;
		switch (TransformTypeIndex)
		{
		case 0: TransFormType = EDatasmithTransformType::Translation; break;
		case 1: TransFormType = EDatasmithTransformType::Rotation; break;
		default: TransFormType = EDatasmithTransformType::Scale;
		}

		FVector* LastValue = InitialValues.Find(TransFormType);
		for (auto& FrameValues : TransformFrames)
		{
			FVector* TransformValue = FrameValues.Value.Find(TransFormType);
			if (!TransformValue)
			{
				TransformValue = LastValue;
			}
			else
			{
				LastValue = TransformValue;
			}
			FVector ConvertedValue = *TransformValue;
			if (TransFormType == EDatasmithTransformType::Scale)
			{
				ConvertedValue = FVector(TransformValue->X, TransformValue->Z, TransformValue->Y);
			}
			else if (TransFormType == EDatasmithTransformType::Translation)
			{
				ConvertedValue = ConvertMelangeDirection(*TransformValue);
			}
			else if (TransFormType == EDatasmithTransformType::Rotation)
			{
				// Copy as we might be reusing a LastValue
				FVector3f TransformValueCopy = (FVector3f)*TransformValue;

				// If the object is in the HPB rotation order, cineware will store its euler rotation
				// as "H, P, B", basically storing the rotations as "YXZ". Lets switch it back to XYZ
				if (RotationOrder == cineware::ROTATIONORDER_HPB)
				{
					Swap(TransformValueCopy.X, TransformValueCopy.Y);
				}

				// TransformValue represents, in radians, the rotations around the C4D axes
				// XRot, YRot, ZRot are rotations around UnrealEditor axes, in the UnrealEditor CS,
				// with the sign given by Quaternion rotations (NOT Rotators)
				FQuat XRot = FQuat(FVector(1, 0, 0), -TransformValueCopy.X);
				FQuat YRot = FQuat(FVector(0, 1, 0),  TransformValueCopy.Z);
				FQuat ZRot = FQuat(FVector(0, 0, 1), -TransformValueCopy.Y);

				// Swap YRot and ZRot in the composition order, as an XYZ order in the C4D CS really means a XZY order in the UnrealEditor CS
				// This effectively converts the rotation order from the C4D CS to the UnrealEditor CS, the sign of the rotations being handled when
				// creating the FQuats
				Swap(YRot, ZRot);

				FQuat FinalQuat;
				switch (RotationOrder)
				{
				case cineware::ROTATIONORDER_XZYGLOBAL:
					FinalQuat = YRot * ZRot * XRot;
					break;
				case cineware::ROTATIONORDER_XYZGLOBAL:
					FinalQuat = ZRot * YRot * XRot;
					break;
				case cineware::ROTATIONORDER_YZXGLOBAL:
					FinalQuat = XRot * ZRot * YRot;
					break;
				case cineware::ROTATIONORDER_ZYXGLOBAL:
					FinalQuat = XRot * YRot * ZRot;
					break;
				case cineware::ROTATIONORDER_YXZGLOBAL:
					FinalQuat = ZRot * XRot * YRot;
					break;
				case cineware::ROTATIONORDER_ZXYGLOBAL:
				case cineware::ROTATIONORDER_HPB:
				default:
					FinalQuat = YRot * XRot * ZRot;
					break;
				}

				// In C4D cameras and lights shoot towards +Z, but in UnrealEditor they shoot towards +X, so fix that with a yaw
				if (Object->GetType() == Olight || Object->GetType() == Ocamera)
				{
					FinalQuat = FinalQuat * FQuat(FVector(0, 0, 1), FMath::DegreesToRadians(-90.0f));
				}

				ConvertedValue = FinalQuat.Euler();
			}
			Animation->AddFrame(TransFormType, FDatasmithTransformFrameInfo(FrameValues.Key, ConvertedValue));
		}
	}

	LevelSequence->AddAnimation(Animation);
}

void FDatasmithC4DImporter::ImportActorHierarchyAnimations(TSharedPtr<IDatasmithActorElement> ActorElement)
{
	for (int32 ChildIndex = 0; ChildIndex < ActorElement->GetChildrenCount(); ++ChildIndex)
	{
		TSharedPtr<IDatasmithActorElement> ChildActorElement = ActorElement->GetChild(ChildIndex);

		ImportAnimations(ChildActorElement);
		ImportActorHierarchyAnimations(ChildActorElement);
	}
}

FVector FDatasmithC4DImporter::GetDocumentDefaultColor()
{
	if (!DefaultDocumentColorLinear.IsSet())
	{
		const int32 DefaultColorType = MelangeGetInt32(C4dDocument, cineware::DOCUMENT_DEFAULTMATERIAL_TYPE);
		switch (DefaultColorType)
		{
		case cineware::DOCUMENT_DEFAULTMATERIAL_TYPE_WHITE: // This says "80% Gray" on the UI
			DefaultDocumentColorLinear = FVector(0.603828f, 0.603828f, 0.603828f);
			break;
		case cineware::DOCUMENT_DEFAULTMATERIAL_TYPE_USER:
			DefaultDocumentColorLinear = MelangeGetColor(C4dDocument, cineware::DOCUMENT_DEFAULTMATERIAL_COLOR);
			break;
		case cineware::DOCUMENT_DEFAULTMATERIAL_TYPE_BLUE: // Intended fall-through. Blue is the default
		default:
			DefaultDocumentColorLinear = FVector(0.099899f, 0.116971f, 0.138432f);
		}
	}

	return DefaultDocumentColorLinear.GetValue();
}

TArray<cineware::TextureTag*> FDatasmithC4DImporter::GetActiveTextureTags(const cineware::BaseObject* Object, const TArray<cineware::TextureTag*>& OrderedTextureTags)
{
	if (!Object)
	{
		return {};
	}

	TArray<cineware::BaseSelect*> OrderedSelectionTags;
	OrderedSelectionTags.Add(nullptr); // "unselected" group

	TMap<FString, cineware::BaseSelect*> SelectionTagsByName;

	// Fetch selection tags, which only affect this polygon
	// The texture tags are fetched when moving down the hierarchy, as texture tags on parents also affect children
	for (cineware::BaseTag* Tag = const_cast<cineware::BaseObject*>(Object)->GetFirstTag(); Tag; Tag = Tag->GetNext())
	{
		if (Tag->GetType() == Tpolygonselection)
		{
			FString SelectionName = MelangeGetString(Tag, cineware::POLYGONSELECTIONTAG_NAME);
			if (!SelectionName.IsEmpty())
			{
				cineware::BaseSelect* BaseSelect = static_cast<cineware::SelectionTag*>(Tag)->GetBaseSelect();
				OrderedSelectionTags.Add(BaseSelect);

				SelectionTagsByName.Add(SelectionName, BaseSelect);
			}
		}
	}

	// If we have multiple texture tags using the same selection, only the latter one will be applied
	// Order is important here: We must scan TextureTags front to back to guarantee that behavior
	TMap<cineware::BaseSelect*, cineware::TextureTag*> ActiveTextureTags;
	for (cineware::TextureTag* TextureTag : OrderedTextureTags)
	{
		cineware::BaseSelect* UsedSelectionTag = nullptr;

		FString UsedSelectionTagName = MelangeGetString(TextureTag, cineware::TEXTURETAG_RESTRICTION);
		if (!UsedSelectionTagName.IsEmpty())
		{
			if (cineware::BaseSelect** FoundSelectionTag = SelectionTagsByName.Find(UsedSelectionTagName))
			{
				UsedSelectionTag = *FoundSelectionTag;
			}
		}

		cineware::BaseList2D* TextureMaterial = TextureTag ? MelangeGetLink(TextureTag, cineware::TEXTURETAG_MATERIAL) : nullptr;

		// Note: If this texture tag is applied without a polygon selection, UsedSelectionTag will be
		// nullptr here, but that is intentional: It's how we signal the "unselected" selection group
		ActiveTextureTags.Add(UsedSelectionTag, TextureTag);
	}

	// Order is important: The polygon groups are created according to the order with which we retrieve our selection tags,
	// so the order with which we return these texture tags must match it exactly
	TArray<cineware::TextureTag*> OrderedActiveTextureTags;
	for (cineware::BaseSelect* SelectionTag : OrderedSelectionTags)
	{
		cineware::TextureTag** TextureTagForSelection = ActiveTextureTags.Find(SelectionTag);
		OrderedActiveTextureTags.Add(TextureTagForSelection ? *TextureTagForSelection : nullptr);
	}

	return OrderedActiveTextureTags;
}

void FDatasmithC4DImporter::RegisterInstancedHierarchy(cineware::BaseObject* InstanceSubObject, cineware::BaseObject* OriginalSubObject)
{
	if (!InstanceSubObject || !OriginalSubObject)
	{
		return;
	}

	InstancedSubObjectsToOriginals.Add(InstanceSubObject, OriginalSubObject);

	RegisterInstancedHierarchy(InstanceSubObject->GetDown(), OriginalSubObject->GetDown());
	RegisterInstancedHierarchy(InstanceSubObject->GetNext(), OriginalSubObject->GetNext());
}

void FDatasmithC4DImporter::RedirectInstancedAnimations()
{
	for (TPair<IDatasmithActorElement*, cineware::BaseObject*>& Entry : ActorElementToAnimationSources)
	{
		if (cineware::BaseObject** OriginalObject = InstancedSubObjectsToOriginals.Find(Entry.Value))
		{
			Entry.Value = *OriginalObject;
		}
	}
}

TSharedPtr<IDatasmithActorElement> FDatasmithC4DImporter::ImportObjectAndChildren(cineware::BaseObject* ActorObject, cineware::BaseObject* DataObject, TSharedPtr<IDatasmithActorElement> ParentActor, const cineware::Matrix& WorldTransformMatrix, const FString& InstancePath, const FString& DatasmithLabel, const TArray<cineware::TextureTag*>& TextureTags)
{
	TSharedPtr<IDatasmithActorElement> ActorElement;
	cineware::Int32 ObjectType = DataObject->GetType();
	cineware::BaseObject* ActorCache = GetBestMelangeCache(ActorObject);
	cineware::BaseObject* DataCache = GetBestMelangeCache(DataObject);
	if (!DataCache)
	{
		DataCache = ActorCache;
	}
	else if (!ActorCache)
	{
		ActorCache = DataCache;
	}

	TOptional<FString> DatasmithName = MelangeObjectID(ActorObject);
	if (!DatasmithName)
	{
		UE_LOGF(LogDatasmithC4DImport, Error, "Could not get the ID of object \"%ls\"", *MelangeObjectName(ActorObject));
		DatasmithName = FString(TEXT("Invalid object"));
	}

	if (!InstancePath.IsEmpty())
	{
		DatasmithName = MD5FromString(InstancePath) + "_" + DatasmithName.GetValue();
	}

	cineware::Matrix NewWorldTransformMatrix = WorldTransformMatrix * ActorObject->GetMl();

	// Fetch actor layer
	FString TargetLayerName;
	bool bActorVisible = true;
	if (cineware::LayerObject* LayerObject = static_cast<cineware::LayerObject*>(MelangeGetLink(ActorObject, cineware::ID_LAYER_LINK)))
	{
		// Do not create actors from invisible layers
		// We may end up creating null actors if the actor is in an invisible layer, and even
		// continuing to import the hierarchy below. This because in C4D if the child is not
		// in the invisible layer, it can actually be visible, and we need to maintain correct transforms
		// and so on.
		// Exceptions are generators: If a cloner is in an invisible layer, the child nodes are always invisible,
		// and also if the cloner is in a visible layer, the child nodes are always visible
		bActorVisible = MelangeGetBool(LayerObject, cineware::ID_LAYER_VIEW);

		TargetLayerName = MelangeObjectName(LayerObject);
	}

	if (bActorVisible)
	{
		bool bSuccess = true;
		bool bImportCache = false;

		if(ObjectType == Oparticle)
		{
			// For particle emitters, we need to mark all the child actors, as those need to have their visibility
			// manually animated to simulate mesh particles spawning and despawning
			MarkActorsAsParticles(ActorObject, ActorCache);
		}

		switch (ObjectType)
		{
		case Ocloner:
		case Oarray:
			//Cloner(Ocloner)
			//	| -CACHE: Null(Onull)
			//	| | -Cube 2(Ocube)
			//	| | | -CACHE: Cube 2(Opolygon)
			//	| | -Cube 1(Ocube)
			//	| | | -CACHE: Cube 1(Opolygon)
			//	| | -Cube 0(Ocube)
			//	| | | -CACHE: Cube 0(Opolygon)
			//	| -Cube(Ocube)

			if (ObjectType == Ocloner && MelangeGetInt32(ActorObject, cineware::MGCLONER_VOLUMEINSTANCES_MODE) != 0)
			{
				// Render/Multi-instance cloner should be ignored
				UE_LOGF(LogDatasmithC4DImport, Warning, "Render-instance or multi-instance Cloners are not supported. Actor '%ls' will be ignored", *MelangeObjectName(ActorObject));
			}
			else if (ObjectType == Oarray && MelangeGetInt32(ActorObject, cineware::ARRAYOBJECT_RENDERINSTANCES) != 0)
			{
				// Render-instance arrays should be ignored
				UE_LOGF(LogDatasmithC4DImport, Warning, "Render-instance Arrays are not supported. Actor '%ls' will be ignored", *MelangeObjectName(ActorObject));
			}
			else
			{
				ActorElement = ImportNullActor(ActorObject, DatasmithName.GetValue(), DatasmithLabel);
				if (DataCache != nullptr && DataCache->GetType() == Onull && AddChildActor(ActorObject, ParentActor, NewWorldTransformMatrix, ActorElement))
				{
					ImportHierarchy(ActorCache->GetDown(), DataCache->GetDown(), ActorElement, NewWorldTransformMatrix, InstancePath, TextureTags);
					return ActorElement;
				}

				bSuccess = false;
			}

			break;

		case Oatomarray:
		case Oconnector:
			// Connector object will have as children the original objects, and its data cache will point at the polygon that results from the actual connect operation,
			// so here we skip that hierarchy and just import that polygon directly
			ActorElement = ImportNullActor(ActorObject, DatasmithName.GetValue(), DatasmithLabel);

			// This will be an empty actor, but we would like to keep it around because its the actor that receives the name of the connect object node itself,
			// while its polygon seems to randomly receive the name of one of the original objects. Keeping the hierarchy like this makes it look exactly like what is
			// shown in C4D if you make a connect object editable
			NamesOfActorsToKeep.Add(ActorElement->GetName());
			if (DataCache != nullptr && AddChildActor(ActorObject, ParentActor, NewWorldTransformMatrix, ActorElement))
			{
				ImportHierarchy(ActorCache, DataCache, ActorElement, NewWorldTransformMatrix, InstancePath, TextureTags);
				return ActorElement;
			}

			bSuccess = false;
			break;

		case Ofracture:
		case ID_MOTIONFRACTUREVORONOI:
		case Osymmetry:
		case Osds: //Sub Division Surface
		case Oboole:
			ActorElement = ImportNullActor(ActorObject, DatasmithName.GetValue() +"0"/*to be different than the cache root*/, DatasmithLabel);
			if (DataCache != nullptr && AddChildActor(ActorObject, ParentActor, NewWorldTransformMatrix, ActorElement)
				&& ImportObjectAndChildren(ActorCache, DataCache, ActorElement, NewWorldTransformMatrix, InstancePath, DatasmithLabel, TextureTags))
			{
				return ActorElement;
			}

			bSuccess = false;
			break;

		case Oinstance:
			if (cineware::BaseObject* InstanceLink = static_cast<cineware::BaseObject*>(MelangeGetLink(DataObject, cineware::INSTANCEOBJECT_LINK)))
			{
				if (TOptional<FString> ObjectID = MelangeObjectID(DataObject))
				{
					// Import the actual instance node
					ActorElement = ImportNullActor(ActorObject, DatasmithName.GetValue(), DatasmithLabel);
					NamesOfActorsToKeep.Add(ActorElement->GetName());

					if (ActorCache != nullptr)
					{
						// Import the cache manually (whatever the instance node is pointing at)
						bImportCache = false;
						ImportHierarchy(ActorCache, DataCache, ActorElement, NewWorldTransformMatrix, ObjectID.GetValue() + InstancePath, TextureTags);

						// We only want to redirect the animations on the subobjects of the original hierarchy (as these can't be interacted with through the instance,
						// so can't have user-set animations). The main Instance node can be independently animated by the user, so we don't want to redirect away from it
						RegisterInstancedHierarchy(ActorCache->GetDown(), InstanceLink->GetDown());
					}
					else
					{
						bSuccess = false;
					}
				}
				else
				{
					bSuccess = false;
				}
			}
			else
			{
				bSuccess = false;
			}
			break;

		case Ospline:
			if (cineware::SplineObject* Spline = static_cast<cineware::SplineObject*>(ActorObject))
			{
				ImportSpline(Spline);
			}
			break;

		default:
			bImportCache = true;
			break;
		}

		if (bSuccess && bImportCache && ActorCache)
		{
			ActorElement = ImportObjectAndChildren(ActorCache, DataCache, nullptr, ActorCache->GetMg(), InstancePath, DatasmithLabel, TextureTags);
		}
		else if (ObjectType == Opolygon)
		{
			cineware::PolygonObject* PolygonObject = static_cast<cineware::PolygonObject*>(DataObject);
			if (Options.bImportEmptyMesh || PolygonObject->GetPolygonCount() > 0)
			{
				TArray<cineware::TextureTag*> ActiveTextureTags = GetActiveTextureTags(PolygonObject, TextureTags);
				if (TSharedPtr<IDatasmithMeshActorElement> MeshActorElement = ImportPolygon(PolygonObject, DatasmithName.GetValue(), DatasmithLabel, ActiveTextureTags))
				{
					ActorElement = MeshActorElement;
				}
				else
				{
					bSuccess = false;
				}
			}
		}
		else if (ObjectType == Ocamera)
		{
			if (TSharedPtr<IDatasmithCameraActorElement> CameraElement = ImportCamera(DataObject, DatasmithName.GetValue(), DatasmithLabel))
			{
				ActorElement = CameraElement;
			}
			else
			{
				bSuccess = false;
			}
		}
		else if (ObjectType == Olight)
		{
			ActorElement = ImportLight(DataObject, DatasmithName.GetValue(), DatasmithLabel);
		}

		if (!bSuccess)
		{
			UE_LOGF(LogDatasmithC4DImport, Error, "Could not import the object \"%ls\"", *MelangeObjectName(ActorObject));
		}

	}

	if (!ActorElement.IsValid())
	{
		ActorElement = ImportNullActor(ActorObject, DatasmithName.GetValue(), DatasmithLabel);
	}

	bool bSuccessfullyAddedChildActor = true;
	if (ParentActor)
	{
		if (!AddChildActor(ActorObject, ParentActor, NewWorldTransformMatrix, ActorElement))
		{
			bSuccessfullyAddedChildActor = false;
			UE_LOGF(LogDatasmithC4DImport, Error, "Could not create the actor for the object \"%ls\"", *MelangeObjectName(ActorObject));
		}
	}

	// Invisible layers will not be imported, so don't use their names
	if (bActorVisible && bSuccessfullyAddedChildActor)
	{
		ActorElement->SetLayer(*TargetLayerName);
	}

	ImportHierarchy(ActorObject->GetDown(), DataObject->GetDown(), ActorElement, NewWorldTransformMatrix, InstancePath, TextureTags);

	return ActorElement;
}

void FDatasmithC4DImporter::ImportHierarchy(cineware::BaseObject* ActorObject, cineware::BaseObject* DataObject, TSharedPtr<IDatasmithActorElement> ParentActor, const cineware::Matrix& WorldTransformMatrix, const FString& InstancePath, const TArray<cineware::TextureTag*>& TextureTags)
{
	while (ActorObject || DataObject)
	{
		if (!DataObject)
		{
			DataObject = ActorObject;
		}
		else if (!ActorObject)
		{
			ActorObject = DataObject;
		}

		// Reset this for every child as texture tags only propagate down, not between siblings
		TArray<cineware::TextureTag*> TextureTagsDown = TextureTags;

		bool SkipObject = false;
		for (cineware::BaseTag* Tag = ActorObject->GetFirstTag(); Tag; Tag = Tag->GetNext())
		{
			cineware::Int32 TagType = Tag->GetType();
			if (TagType == Tannotation)
			{
				FString AnnotationLabel = MelangeGetString(Tag, 10014);
				if (AnnotationLabel.Compare("EXCLUDE", ESearchCase::IgnoreCase) == 0)
				{
					SkipObject = true;
					break;
				}
			}
			else if (TagType == Ttexture)
			{
				TextureTagsDown.Add(static_cast<cineware::TextureTag*>(Tag));
			}
		}

		if (!SkipObject)
		{
			FString DatasmithLabel = FDatasmithUtils::SanitizeObjectName(MelangeObjectName(ActorObject));
			ImportObjectAndChildren(ActorObject, DataObject, ParentActor, WorldTransformMatrix, InstancePath, DatasmithLabel, TextureTagsDown);
		}

		ActorObject = ActorObject->GetNext();
		DataObject = DataObject->GetNext();
	}
}

TSharedPtr<IDatasmithMeshElement> FDatasmithC4DImporter::ImportMesh(cineware::PolygonObject* PolyObject, const FString& DatasmithMeshName, const FString& DatasmithLabel)
{
	if (!PolyObject)
	{
		return nullptr;
	}

	cineware::Int32 PointCount = PolyObject->GetPointCount();
	cineware::Int32 PolygonCount = PolyObject->GetPolygonCount();
	if (PointCount == 0 || PolygonCount == 0)
	{
		// Missing points or triangles. Skipping this polygon object
		return nullptr;
	}

	const cineware::Vector* Points = PolyObject->GetPointR();
	const cineware::CPolygon* Polygons = PolyObject->GetPolygonR();
	if (!Points || !Polygons)
	{
		// Something wrong happened. Skipping this polygon object
		ensure(false);
		return nullptr;
	}

	// Get vertex normals
	cineware::Vector32* Normals = nullptr;
	if (PolyObject->GetTag(Tphong))
	{
		Normals = PolyObject->CreatePhongNormals();
	}

	// Collect all UV channels and material slot information for this PolygonObject
	TArray<cineware::ConstUVWHandle> UVWTagsData;
	TArray<cineware::BaseSelect*> SelectionTags;
	SelectionTags.Add(nullptr); // The "unselected" group

	for (cineware::BaseTag* Tag = PolyObject->GetFirstTag(); Tag; Tag = Tag->GetNext())
	{
		cineware::Int32 TagType = Tag->GetType();
		if (TagType == Tuvw)
		{
			cineware::UVWTag* UVWTag = static_cast<cineware::UVWTag*>(Tag);
			if (UVWTag->GetDataCount() == PolygonCount)
			{
				UVWTagsData.Add(UVWTag->GetDataAddressR());
			}
			else
			{
				return TSharedPtr<IDatasmithMeshElement>();
			}
		}
		else if (TagType == Tpolygonselection)
		{
			FString SelectionName = MelangeGetString(Tag, cineware::POLYGONSELECTIONTAG_NAME);
			if (!SelectionName.IsEmpty())
			{
				SelectionTags.Add(static_cast<cineware::SelectionTag*>(Tag)->GetBaseSelect());
			}
		}
	}

	if (UVWTagsData.Num() > MAX_STATIC_TEXCOORDS-1)
	{
		UE_LOGF(LogDatasmithC4DImport, Error, "Mesh '%ls' has %d UV channels! Only the first %d will be used", *DatasmithLabel, MAX_STATIC_TEXCOORDS-1, UVWTagsData.Num());
		UVWTagsData.SetNum(MAX_STATIC_TEXCOORDS-1);
	}

	int32 NumSlots = SelectionTags.Num();

	// Create MeshDescription
	FMeshDescription MeshDescription;
	DatasmithMeshHelper::PrepareAttributeForStaticMesh(MeshDescription);
	MeshDescription.Empty();

	FStaticMeshAttributes StaticMeshAttributes(MeshDescription);
	TVertexAttributesRef<FVector3f> VertexPositions = StaticMeshAttributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = StaticMeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = StaticMeshAttributes.GetPolygonGroupMaterialSlotNames();

	// Reserve space for attributes. These might not be enough as some of these polygons might be quads or n-gons, but its better than nothing
	MeshDescription.ReserveNewVertices(PointCount);
	MeshDescription.ReserveNewVertexInstances(PolygonCount);
	MeshDescription.ReserveNewEdges(PolygonCount);
	MeshDescription.ReserveNewPolygons(PolygonCount);
	MeshDescription.ReserveNewPolygonGroups(NumSlots);

	// At least one UV set must exist.
	int32 UVChannelCount = UVWTagsData.Num();
	if (!VertexInstanceUVs.IsValid())
	{
		return TSharedPtr<IDatasmithMeshElement>();
	}
	VertexInstanceUVs.SetNumChannels(FMath::Max(1, UVChannelCount));

	// Vertices
	if (!VertexPositions.IsValid())
	{
		return nullptr;
	}

	for (int32 PointIndex = 0; PointIndex < PointCount; ++PointIndex)
	{
		FVertexID NewVertexID = MeshDescription.CreateVertex();
		// We count on this check when creating polygons
		if (NewVertexID.GetValue() != PointIndex)
		{
			// Something wrong happened
			return nullptr;
		}

		VertexPositions[NewVertexID] = (FVector3f)ConvertMelangePosition(Points[PointIndex]);
	}

	// Create one material slot per polygon selection tag (including the "unselected" group)
	for (int32 SlotIndex = 0; SlotIndex < NumSlots; ++SlotIndex)
	{
		FPolygonGroupID PolyGroupId = MeshDescription.CreatePolygonGroup();
		PolygonGroupImportedMaterialSlotNames[PolyGroupId] = DatasmithMeshHelper::DefaultSlotName(SlotIndex);
	}

	// Vertex indices in a quad or a triangle
	TArray<int32> QuadIndexOffsets = { 0, 1, 3, 1, 2, 3 };
	TArray<int32> TriangleIndexOffsets = { 0, 1, 2 };
	TArray<int32>* IndexOffsets;

	// We have to pass 3 instance IDs at a time to CreatePolygon, so we must copy
	TArray<FVertexInstanceID> IDsCopy;
	IDsCopy.SetNumUninitialized(3);
	TArray<FVector> QuadNormals;
	QuadNormals.SetNumZeroed(4);
	TArray<FVector2f> QuadUVs;
	QuadUVs.SetNumZeroed(4);

	// Used to check for degenerate triangles
	TArray<FVertexID> TriangleVertices;
	TriangleVertices.SetNumZeroed(3);
	TArray<FVector> TriangleVertexPositions;
	TriangleVertexPositions.SetNumZeroed(3);

	// Create polygons
	for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; ++PolygonIndex)
	{
		const cineware::CPolygon& Polygon = Polygons[PolygonIndex];

		// Check if we're a triangle or a quad
		if (Polygon.c == Polygon.d)
		{
			IndexOffsets = &TriangleIndexOffsets;
		}
		else
		{
			IndexOffsets = &QuadIndexOffsets;
		}

		// Get which vertices we'll use for this polygon
		TArray<FVertexID> VerticesForPolygon;
		for ( int32 VertexIndexOffset : *IndexOffsets )
		{
			VerticesForPolygon.Emplace( Polygon[ VertexIndexOffset ] );
		}

		// Create vertex instances for valid triangles
		TArray<FVertexInstanceID> VertexInstances;
		for ( int32 TriangleIndex = 0; TriangleIndex < VerticesForPolygon.Num() / 3; ++TriangleIndex )
		{
			for ( int32 VertexIndex = 0; VertexIndex < 3; ++VertexIndex )
			{
				FVertexID VertID = VerticesForPolygon[ TriangleIndex * 3 + VertexIndex ];

				TriangleVertices[ VertexIndex ] = VertID;
				TriangleVertexPositions[ VertexIndex ] = (FVector)VertexPositions[ VertID ];
			}

			// Check if those vertices lead to degenerate triangles first, to prevent us from ever adding unused data to the MeshDescription
			FVector RawNormal = ( ( TriangleVertexPositions[ 1 ] - TriangleVertexPositions[ 2 ] ) ^ ( TriangleVertexPositions[ 0 ] - TriangleVertexPositions[ 2 ] ) );
			if ( RawNormal.SizeSquared() < UE_SMALL_NUMBER )
			{
				continue;
			}

			// Valid triangle, create vertex instances for it
			for ( const FVertexID VertID : TriangleVertices )
			{
				VertexInstances.Add( MeshDescription.CreateVertexInstance( VertID ) );
			}
		}

		// Fetch cineware polygon normals (always 4, even if triangle)
		if (Normals)
		{
			for (int32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
			{
				QuadNormals[VertexIndex] = ConvertMelangeDirection(Normals[PolygonIndex * 4 + VertexIndex]);
			}
			// Set normals
			for (int32 VertexCount = 0; VertexCount < VertexInstances.Num(); ++VertexCount)
			{
				FVertexInstanceID& VertInstanceID = VertexInstances[VertexCount];
				int32 VertexIDInQuad = (*IndexOffsets)[VertexCount];

				VertexInstanceNormals.Set(VertInstanceID, (FVector3f)QuadNormals[VertexIDInQuad]);
			}
		}

		// UVs
		for (int32 ChannelIndex = 0; ChannelIndex < UVChannelCount; ++ChannelIndex)
		{
			cineware::ConstUVWHandle UVWTagData = UVWTagsData[ChannelIndex];
			cineware::UVWStruct UVWStruct;
			cineware::UVWTag::Get(UVWTagData, PolygonIndex, UVWStruct);
			cineware::Vector* UVs = &UVWStruct.a;

			// Fetch cineware UVs
			for (int32 VertexIndex = 0; VertexIndex < 4; ++VertexIndex)
			{
				cineware::Vector& PointUVs = UVs[VertexIndex];
				FVector2f& UnrealUVs = QuadUVs[VertexIndex];

				if (PointUVs.z != 0.0f && PointUVs.z != 1.0f)
				{
					UnrealUVs.X = static_cast<float>(PointUVs.x / PointUVs.z);
					UnrealUVs.Y = static_cast<float>(PointUVs.y / PointUVs.z);
				}
				else
				{
					UnrealUVs.X = static_cast<float>(PointUVs.x);
					UnrealUVs.Y = static_cast<float>(PointUVs.y);
				}

				if (UnrealUVs.ContainsNaN())
				{
					UnrealUVs.Set(0, 0);
				}
			}
			// Set UVs
			for (int32 VertexCount = 0; VertexCount < VertexInstances.Num(); ++VertexCount)
			{
				FVertexInstanceID& VertInstanceID = VertexInstances[VertexCount];
				int32 VertexIDInQuad = (*IndexOffsets)[VertexCount];

				VertexInstanceUVs.Set(VertInstanceID, ChannelIndex, QuadUVs[VertexIDInQuad]);
			}
		}

		// Find which selection tag (and so which material slot and polygon group) we belong to.
		// Note that if we don't find any we end up in SlotIndex 0, which is the "unselected" group.
		// Also note that we already receive just one texture tag per selection
		int32 SlotIndex = NumSlots - 1;
		for (; SlotIndex > 0; --SlotIndex)
		{
			cineware::BaseSelect* SelectionTag = SelectionTags[SlotIndex];

			if (SelectionTag && SelectionTag->IsSelected(PolygonIndex))
			{
				break;
			}
		}

		// Create a triangle for each 3 vertex instance IDs we have
		for (int32 TriangleIndex = 0; TriangleIndex < VertexInstances.Num() / 3; ++TriangleIndex)
		{
			FMemory::Memcpy(IDsCopy.GetData(), VertexInstances.GetData() + TriangleIndex * 3, sizeof(FVertexInstanceID) * 3);

			// Invert winding order for triangles
			IDsCopy.Swap(0, 2);

			const FPolygonID NewPolygonID = MeshDescription.CreatePolygon(FPolygonGroupID(SlotIndex), IDsCopy);

			// Fill in the polygon's Triangles - this won't actually do any polygon triangulation as we always give it triangles
			MeshDescription.ComputePolygonTriangulation(NewPolygonID);
		}
	}

	int32 NumPolygons = MeshDescription.Polygons().Num();
	TArray<uint32> ZeroedFaceSmoothingMask;
	ZeroedFaceSmoothingMask.SetNumZeroed(NumPolygons);
	FStaticMeshOperations::ConvertSmoothGroupToHardEdges(ZeroedFaceSmoothingMask, MeshDescription);

	if (Normals)
	{
		cineware::DeleteMem(Normals);
	}

	TSharedRef<IDatasmithMeshElement> MeshElement = FDatasmithSceneFactory::CreateMesh(*DatasmithMeshName);
	MeshElement->SetLabel(*DatasmithLabel);

	MeshElementToMeshDescription.Add(&MeshElement.Get(), MoveTemp(MeshDescription));

	DatasmithScene->AddMesh(MeshElement);
	return MeshElement;
}

void FDatasmithC4DImporter::GetGeometriesForMeshElementAndRelease(const TSharedRef<IDatasmithMeshElement> MeshElement, TArray<FMeshDescription>& OutMeshDescriptions)
{
	if (FMeshDescription* MeshDesc = MeshElementToMeshDescription.Find(&MeshElement.Get()))
	{
		OutMeshDescriptions.Add(MoveTemp(*MeshDesc));
	}
}

TSharedPtr<IDatasmithLevelSequenceElement> FDatasmithC4DImporter::GetLevelSequence()
{
	return LevelSequence;
}

bool FDatasmithC4DImporter::OpenFile(const FString& InFilename)
{
	SCOPE_CYCLE_COUNTER(STAT_C4DImporter_LoadFile)

	using namespace cineware;

	if (!FPaths::FileExists(InFilename))
	{
		return false;
	}

	C4dDocument = NewObj(BaseDocument);
	if (C4dDocument == nullptr)
	{
		return false;
	}

	HyperFile *C4Dfile = NewObj(HyperFile);
	if (!C4Dfile)
	{
		DeleteObj(C4dDocument);
		return false;
	}

	if (C4Dfile->Open(DOC_IDENT, TCHAR_TO_ANSI(*InFilename), FILEOPEN_READ))
	{
		// Extra nullptr check because static analysis tool doesn't understand that this was already checked
		bool bSuccess = C4dDocument != nullptr && C4dDocument->ReadObject(C4Dfile, true);

		int64 LastPos = static_cast<int64>(C4Dfile->GetPosition());
		int64 Length = static_cast<int64>(C4Dfile->GetLength());
		int64 Version = static_cast<int64>(C4Dfile->GetFileVersion());
		FILEERROR Error = C4Dfile->GetError();

		if (bSuccess)
		{
			UE_LOGF(LogDatasmithC4DImport, Log, "Melange SDK successfully read the file '%ls' (read %lld out of %lld bytes, version %lld)", *InFilename, LastPos, Length, Version);
		}
		else
		{
			UE_LOGF(LogDatasmithC4DImport, Warning, "Melange SDK did not read the entire file '%ls' (read %lld out of %lld bytes, version %lld, error code: %d). Imported scene may contain errors or missing data.", *InFilename, LastPos, Length, Version, Error);
		}
	}
	else
	{
		UE_LOGF(LogDatasmithC4DImport, Error, "Cannot open file '%ls'", *InFilename);
		DeleteObj(C4Dfile);
		DeleteObj(C4dDocument);
		return false;
	}

	C4dDocumentFilename = InFilename;

	C4Dfile->Close();
	DeleteObj(C4Dfile);

	if (C4dDocument != nullptr && !C4dDocument->HasCaches())
	{
		FText ErrorMsg = FText::Format(LOCTEXT("C4DNotSavedForMelange", "The file '{0}' was not saved for Melange."), FText::FromString(C4dDocumentFilename));
		UE_LOGF(LogDatasmithC4DImport, Warning, "%ls", *ErrorMsg.ToString());

		FNotificationInfo NotificationInfo(ErrorMsg);
		NotificationInfo.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}

	return true;
}

cineware::BaseObject* FDatasmithC4DImporter::FindMelangeObject(const FString& SearchObjectID, cineware::BaseObject* Object)
{
	cineware::BaseObject* FoundObject = nullptr;

	while (Object && !FoundObject)
	{
		if (TOptional<FString> ObjectID = MelangeObjectID(Object))
		{
			if (ObjectID.GetValue() == SearchObjectID)
			{
				FoundObject = Object;
				break;
			}
		}
		else
		{
			//The object is invalid or we could not find its ID.
			break;
		}

		FoundObject = FindMelangeObject(SearchObjectID, Object->GetDown());
		Object = Object->GetNext();
	}

	return FoundObject;
}

cineware::BaseObject* FDatasmithC4DImporter::GoToMelangeHierarchyPosition(cineware::BaseObject* Object, const FString& HierarchyPosition)
{
	if (Object)
	{
		int32 SeparatorIndex = 0;
		bool SeparatorFound = HierarchyPosition.FindChar('_', SeparatorIndex);
		int32 IndexFromRoot = FCString::Atoi(SeparatorFound ? *HierarchyPosition.Left(SeparatorIndex) : *HierarchyPosition);
		while (Object && IndexFromRoot > 0)
		{
			Object = Object->GetNext();
			IndexFromRoot--;
		}

		if (SeparatorFound && HierarchyPosition.Len() > SeparatorIndex + 1)
		{
			FString NextHierarchyPosition = HierarchyPosition.Right(HierarchyPosition.Len() - SeparatorIndex - 1);
			if (NextHierarchyPosition.Left(2) == "C_")
			{
				Object = GoToMelangeHierarchyPosition(GetBestMelangeCache(Object), NextHierarchyPosition.Right(NextHierarchyPosition.Len() - 2));
			}
			else if(Object)
			{
				Object = GoToMelangeHierarchyPosition(Object->GetDown(), NextHierarchyPosition);
			}
		}
	}
	return Object;
}

bool FDatasmithC4DImporter::ProcessScene()
{
	// Cinema 4D Document settings
	MelangeFPS = static_cast<cineware::Float>(MelangeGetInt32(C4dDocument, cineware::DOCUMENT_FPS));
	if(MelangeFPS == 0.0f)
	{
		UE_LOGF(LogDatasmithC4DImport, Error, "DOCUMENT_FPS not found");
		return false;
	}
	MelangeColorProfile = MelangeGetInt32(C4dDocument, cineware::DOCUMENT_COLORPROFILE);
	cineware::RenderData* RenderData = C4dDocument->GetActiveRenderData();
	if (!RenderData)
	{
		UE_LOGF(LogDatasmithC4DImport, Error, "Active Render Data not found");
		return false;
	}

	// Materials
	ImportedTextures.Empty();
	if (!ImportMaterialHierarchy(C4dDocument->GetFirstMaterial()))
	{
		return false;
	}
	ImportedTextures.Empty();

	// Actors
	// Need a RootActor for RemoveEmptyActors and to make AddChildActor agnostic to actor hierarchy level
	TSharedPtr<IDatasmithActorElement> RootActor = FDatasmithSceneFactory::CreateActor(TEXT("RootActor"));
	DatasmithScene->AddActor(RootActor);
	TArray<cineware::TextureTag*> TextureTags;
	ImportHierarchy(C4dDocument->GetFirstObject(), C4dDocument->GetFirstObject(), RootActor, cineware::Matrix(), "", TextureTags);

	// Animations
	LevelSequence = FDatasmithSceneFactory::CreateLevelSequence(DatasmithScene->GetName());
	LevelSequence->SetFrameRate(static_cast<float>(MelangeFPS));
	DatasmithScene->AddLevelSequence(LevelSequence.ToSharedRef());
	RedirectInstancedAnimations();
	ImportActorHierarchyAnimations(RootActor);

	// Processing
	FC4DImporterImpl::KeepParentsOfAnimatedNodes(RootActor, NamesOfActorsToKeep);
	FC4DImporterImpl::RemoveEmptyActors(DatasmithScene, NamesOfActorsToKeep);
	DatasmithScene->RemoveActor(RootActor, EDatasmithActorRemovalRule::KeepChildrenAndKeepRelativeTransform);

#if WITH_EDITOR
	if (Options.bExportToUDatasmith)
	{
		SceneExporterRef = TSharedRef<FDatasmithSceneExporter>(new FDatasmithSceneExporter);
		SceneExporterRef->PreExport();
		FString SceneName = FDatasmithUtils::SanitizeFileName(FDatasmithUtils::SanitizeObjectName(FPaths::GetBaseFilename(C4dDocumentFilename)));
		SceneExporterRef->SetName(*SceneName);
		SceneExporterRef->SetOutputPath(*FPaths::GetPath(C4dDocumentFilename));
		SceneExporterRef->Export(DatasmithScene);
	}
#endif //WITH_EDITOR

	return true;
}

void FDatasmithC4DImporter::UnloadScene()
{
	DeleteObj(C4dDocument);
}

// Traverse the LayerObject hierarchy adding visible layer names to VisibleLayers
void RecursivelyParseLayers(cineware::LayerObject* CurrentLayer, TSet<FName>& VisibleLayers)
{
	if (CurrentLayer == nullptr)
	{
		return;
	}

	FString Name = MelangeObjectName(CurrentLayer);

	cineware::GeData Data;
	if (MelangeGetBool(CurrentLayer, cineware::ID_LAYER_VIEW))
	{
		VisibleLayers.Add(FName(*Name));
	}

	RecursivelyParseLayers(CurrentLayer->GetDown(), VisibleLayers);
	RecursivelyParseLayers(CurrentLayer->GetNext(), VisibleLayers);
}

#undef LOCTEXT_NAMESPACE

#endif //_MELANGE_SDK_
