// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MELANGE_SDK_

#ifdef UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES
#undef UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES
#endif
#define UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES 0

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "MeshDescription.h"
#include "Stats/Stats.h"
#include "UObject/StrongObjectPtr.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include "DatasmithTranslator.h"
#include "DatasmithC4DMelangeSDKEnterGuard.h"
#include "c4d.h"
#include "c4d_browsecontainer.h"
#include "DatasmithC4DMelangeSDKLeaveGuard.h"

class FDatasmithSceneExporter;
class IDatasmithActorElement;
class IDatasmithCameraActorElement;
class IDatasmithLevelSequenceElement;
class IDatasmithLightActorElement;
class IDatasmithMaterialInstanceElement;
class IDatasmithMeshActorElement;
class IDatasmithMeshElement;
class IDatasmithScene;
class IDatasmithTextureElement;
class UDatasmithStaticMeshImportData;
enum class EDatasmithTextureMode : uint8;
struct FCraneCameraAttributes;
struct FDatasmithImportContext;
struct FRichCurve;


DECLARE_STATS_GROUP(TEXT("C4DImporter"), STATGROUP_C4DImporter, STATCAT_Advanced);

struct FDatasmithC4DImportOptions
{
	bool bImportEmptyMesh = false;
	bool bOptimizeEmptySingleChildActors = false;
	bool bAlwaysGenerateNormals = false;
	float ScaleVertices = 1.0;
	bool bExportToUDatasmith = false;
};

class FDatasmithC4DImporter
{
public:
	FDatasmithC4DImporter(TSharedRef<IDatasmithScene>& OutScene, FDatasmithC4DImportOptions& InOptions);
	~FDatasmithC4DImporter();

	/** Updates the used import options to InOptions */
	void SetImportOptions(FDatasmithC4DImportOptions& InOptions);

	/** Open and load a .4cd file into C4dDocument */
	bool OpenFile(const FString& InFilename);

	/** Parse the scene contained in the previously opened file and process its content according to parameters from incoming context */
	bool ProcessScene();

	/** Unload cineware resources after importing */
	void UnloadScene();

	void GetGeometriesForMeshElementAndRelease(const TSharedRef<IDatasmithMeshElement> MeshElement, TArray<FMeshDescription>& OutMeshDescriptions);
	TSharedPtr<IDatasmithLevelSequenceElement> GetLevelSequence();

	/** Finds the most derived cache for a cineware object. That will be e.g. a polygon cache or a deformed polygon cache, if it has one */
	cineware::BaseObject* GetBestMelangeCache(cineware::BaseObject* Object);

	/** Generates a unique identifier string for a cineware object based on the object's name and its position in the hierarchy */
	TOptional<FString> MelangeObjectID(cineware::BaseObject* Object);

	/** Searches the cineware object hierarchy for a cineware::BaseObject that has a MelangeObjectID equal to SearchObjectID */
	cineware::BaseObject* FindMelangeObject(const FString& SearchObjectID, cineware::BaseObject* Object);

	/** Fetch the object corresponding to the position encoded in HierarchyPosition, starting from Object */
	cineware::BaseObject* GoToMelangeHierarchyPosition(cineware::BaseObject* Object, const FString& HierarchyPosition);

	/** Gets all cineware objects that are part of the hierarchy of InstanceRoot. Used to identify child hierarchies of Oinstance objects */
	//const TArray<cineware::BaseObject*>& GetMelangeInstanceObjects(cineware::BaseObject* InstanceRoot);

	/**
	 * Marks actors children of EmitterObject as ParticleActors, so that they can receive an artificial visibility animation track to
	 * emulate the look of the particle spawning and despawning
	 */
	void MarkActorsAsParticles(cineware::BaseObject* EmitterObject, cineware::BaseObject* EmittersCache);

	/**
	 * Import cineware objects into Datasmith elements. Assets like meshes, materials and textures are added to the Datasmith scene directly,
	 * while actors are merely returned and must be added as children to scene actors or added to the scene manually.
	 *
	 * Returns a null TSharedPtr if an error occurred during the import process.
	 */
	TSharedPtr<IDatasmithActorElement> ImportNullActor(cineware::BaseObject* Object, const FString& DatasmithName, const FString& DatasmithLabel);
	TSharedPtr<IDatasmithLightActorElement> ImportLight(cineware::BaseObject* LightObject, const FString& DatasmithName, const FString& DatasmithLabel);
	TSharedPtr<IDatasmithCameraActorElement> ImportCamera(cineware::BaseObject* CameraObject, const FString& DatasmithName, const FString& DatasmithLabel);
	TSharedPtr<IDatasmithMeshActorElement> ImportPolygon(cineware::PolygonObject* PolyObject, const FString& DatasmithName, const FString& DatasmithLabel, const TArray<cineware::TextureTag*>& TextureTags);
	TSharedPtr<IDatasmithMaterialInstanceElement> ImportMaterial(cineware::Material* InC4DMaterialPtr);
	TSharedPtr<IDatasmithMaterialInstanceElement> ImportSimpleColorMaterial(cineware::BaseObject* Object, int32 UseColor);
	TSharedPtr<IDatasmithTextureElement> ImportTexture(const FString& TexturePath, EDatasmithTextureMode TextureMode);
	TSharedPtr<IDatasmithMeshElement> ImportMesh(cineware::PolygonObject* PolyObject, const FString& DatasmithMeshName, const FString& DatasmithLabel);

	/** Parses the spline and its cache into SplineCurves so that it can be used as paths for animation later */
	void ImportSpline(cineware::SplineObject* ActorObject);

	/** Traverse the cineware material hierarchy contained in the c4d file and import each into IDatasmithMaterialInstanceElements */
	bool ImportMaterialHierarchy(cineware::BaseMaterial* InC4DMaterialPtr);

	/** Uses ActorElementToC4DObject to find the corresponding cineware object for ActorElement and adds all of its animations to LevelSequence */
	void ImportAnimations(TSharedPtr<IDatasmithActorElement> ActorElement);

	/** Traverse the Datasmith scene's IDatasmithActorElement hierarchy and import all animations for the corresponding cineware actors*/
	void ImportActorHierarchyAnimations(TSharedPtr<IDatasmithActorElement> ActorElement);

	/** Searches for the first valid texture used by BaseShader */
	FString GetBaseShaderTextureFilePath(cineware::BaseList2D* BaseShader);

	/**
	 * Generates a new copy of the IDatasmithMaterialInstanceElement with name InMaterialID and alter its properties to match values retrieved
	 * from InTextureTag, and adds the new material to the Datasmith scene. This is used because texture tags are closer to material
	 * instances, and may have different "overrides" for each property
	 */
	FString CustomizeMaterial(const FString& InMaterialID, const FString& InMeshID, cineware::TextureTag* InTextureTag);

	/**
	 * Creates customized materials if necessary, and returns a map from material slot indices to material names
	 */
	TMap<int32, FString> GetCustomizedMaterialAssignment(const FString& DatasmithMeshName, const TArray<cineware::TextureTag*>& TextureTags);

	/** Imports a cineware actor, which might involve parsing another small hierarchy of subnodes and deformers*/
	TSharedPtr<IDatasmithActorElement> ImportObjectAndChildren(cineware::BaseObject* ActorObject, cineware::BaseObject* DataObject, TSharedPtr<IDatasmithActorElement> ParentActor, const cineware::Matrix& WorldTransformMatrix, const FString& InstancePath, const FString& DatasmithLabel, const TArray<cineware::TextureTag*>& TextureTags);

	/** Traverse the cineware actor hierarchy importing all nodes */
	void ImportHierarchy(cineware::BaseObject* ActorObject, cineware::BaseObject* DataObject, TSharedPtr<IDatasmithActorElement> ParentActor, const cineware::Matrix& WorldTransformMatrix, const FString& InstancePath, const TArray<cineware::TextureTag*>& TextureTags);

	/**
	 * Adds Actor as a child of ParentActor using the corresponding WorldTransformMatrix. Object is the corresponding cineware Object to Actor.
	 * This will also do the necessary coordinate system conversions between cineware and Unreal.
	 */
	bool AddChildActor(cineware::BaseObject* Object, TSharedPtr<IDatasmithActorElement> ParentActor, cineware::Matrix WorldTransformMatrix, const TSharedPtr<IDatasmithActorElement>& Actor);

	/** Event for when a C4D document is about to be opened for translation */
	DECLARE_EVENT_TwoParams(FDatasmithC4DTranslator, FPreTranslateEvent, cineware::BaseDocument*, const FString&)
	static FPreTranslateEvent& OnPreTranslate() { return PreTranslateEvent; }

	cineware::BaseDocument* C4dDocument = nullptr;
	FString C4dDocumentFilename;

private:


	FVector GetDocumentDefaultColor();

	/** Returns the TextureTags that should affect this object. May check parent objects, so relies on CachesOriginalObject */
	TArray<cineware::TextureTag*> GetActiveTextureTags(const cineware::BaseObject* Object, const TArray<cineware::TextureTag*>& OrderedTextureTags);

	/** Removes from Context->Scene all empty actors that have a single child */
	//void RemoveEmptyActors();

	/** Traverses the original and instanced hierarchy simultaneously and register links between instanced objects and their originals */
	void RegisterInstancedHierarchy(cineware::BaseObject* InstanceRoot, cineware::BaseObject* OriginalRoot);

	/**
	 * Replaces the values of ActorElementToAnimationSources to point to the original objects, in case they are instanced objects.
	 * This is required because we need to parse the instanced hierarchy to fetch the correct polygons and materials, but Cinema 4D only places animations
	 * on the original hierarchy.
	 */
	void RedirectInstancedAnimations();

	/** Storage of FMeshDescriptions until they're retrieved by GetGeometriesForMeshElement */
	TMap<IDatasmithMeshElement*, FMeshDescription> MeshElementToMeshDescription;

	/** Storage of imported spline data to be used exclusively for importing animations that follow spline paths */
	TMap<cineware::SplineObject*, TArray<FRichCurve>> SplineCurves;

	/** Storage of created materials used by CustomizeMaterial to create new "material instances" */
	TMap<FString, TSharedPtr<IDatasmithMaterialInstanceElement>> MaterialNameToMaterialElement;

	/** Cache meshes by hash to promote reusing StaticMeshes */
	TMap<FString, TSharedRef<IDatasmithMeshElement>> PolygonHashToMeshElement;

	/** Cache to prevent us from importing the same texture in the same mode more than once (mode is encoded in the FString as well) */
	TMap<FString, TSharedPtr<IDatasmithTextureElement>> ImportedTextures;

	/** Storage of all parsed actors from the cineware document, used so we can import all animations afterwards */
	TMap<IDatasmithActorElement*, cineware::BaseObject*> ActorElementToAnimationSources;

	/** Maps an instance to the corresponding original node, used so that we can redirect animations to the original nodes */
	TMap<cineware::BaseObject*, cineware::BaseObject*> InstancedSubObjectsToOriginals;

	/** Keeps track of the owners of every cineware cache object so we can climb the hierarchy upwards */
	TMap<cineware::BaseObject*, cineware::BaseObject*> CachesOriginalObject;

	/** Stores all FCraneCameraAttributes for each camera */
	TMap<cineware::BaseObject*, TSharedRef<FCraneCameraAttributes>> CraneCameraToAttributes;

	/** Melange actors that are actually baked 'mesh particles' and need to receive an extra visibility track in ImportAnimations */
	TSet<cineware::BaseObject*> ParticleActors;

	/** Caches to make sure we don't have any actor name collisions */
	TSet<FString> NamesOfAllActors;

	/** Names of IDatasmithActorElements that shouldn't be removed when optimizing the scene */
	TSet<FString> NamesOfActorsToKeep;

	FDatasmithC4DImportOptions Options;

	/** Where all actor animations are imported into when parsing the scene */
	TSharedPtr<IDatasmithLevelSequenceElement> LevelSequence;

	/** Output Datasmith scene */
	TSharedRef<IDatasmithScene> DatasmithScene;

	/** Can be used to also export the imported scene in a .udatasmith format during import */
	TSharedPtr<FDatasmithSceneExporter> SceneExporterRef;

	TOptional<FVector> DefaultDocumentColorLinear;
	
	static FPreTranslateEvent PreTranslateEvent;
};

#endif // _MELANGE_SDK_