// Copyright Epic Games, Inc. All Rights Reserved.

// Enter Datasmith platform include gard.
#define MAXON_TARGET_RELEASE 1
#define MAXON_TARGET_64BIT 1

// These UE defines conflict with the maxon headers, so remove them
#ifdef LIGHT_TYPE_DIRECTIONAL
#undef LIGHT_TYPE_DIRECTIONAL
#undef LIGHT_TYPE_POINT
#undef LIGHT_TYPE_SPOT
#undef LIGHT_TYPE_RECT
#undef LIGHT_TYPE_MAX
#endif

#include "HAL/Platform.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"

#define MAXON_TARGET_WINDOWS 1

// Back up some preprocessor identifiers.
__pragma(push_macro("PI"))
__pragma(push_macro("BYTE_MAX"))
__pragma(push_macro("_HAS_EXCEPTIONS"))
__pragma(push_macro("__LP64__"))
__pragma(push_macro("__APPLE__"))

__pragma(push_macro("GCC34T"))
#define GCC34T this->

#define __LP64__  0
#define __APPLE__ 0

#undef PI

// Make Melange private_symbols.h build.
#undef BYTE_MAX

__pragma(warning(push))
__pragma(warning(disable: 6297)) /* cineware\20.004_rbmelange20.0_259890\includes\c4d_drawport.h(276): Arithmetic overflow:  32-bit value is shifted, then cast to 64-bit value.  Results might not be an expected value. */

#elif PLATFORM_MAC

#define MAXON_TARGET_OSX 1

#endif // PLATFORM_WINDOWS

#ifdef __clang__

#pragma push_macro("GCC34T")
#pragma push_macro("PI")

#define GCC34T this->
#undef PI

_Pragma( "clang diagnostic push" )
_Pragma( "clang diagnostic ignored \"-Wdeprecated-declarations\"" )
_Pragma( "clang diagnostic ignored \"-Wundef\"")

#endif // __clang__


