// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Not included directly

#ifdef __clang__

_Pragma( "clang diagnostic pop" )

#pragma pop_macro("GCC34T")
#pragma pop_macro("PI")

#endif // __clang__

#if PLATFORM_WINDOWS

#undef MAXON_TARGET_WINDOWS

__pragma(warning(pop))

// Restore some preprocessor identifiers.
__pragma(pop_macro("PI"))
__pragma(pop_macro("BYTE_MAX"))
__pragma(pop_macro("_HAS_EXCEPTIONS"))
__pragma(pop_macro("__LP64__"))
__pragma(pop_macro("__APPLE__"))

__pragma(pop_macro("GCC34T"))

// Leave Datasmith platform include guard.
#include "Windows/HideWindowsPlatformTypes.h"

#elif PLATFORM_MAC

#undef MAXON_TARGET_OSX

#endif // PLATFORM_WINDOWS

#undef MAXON_TARGET_RELEASE
#undef MAXON_TARGET_64BIT

