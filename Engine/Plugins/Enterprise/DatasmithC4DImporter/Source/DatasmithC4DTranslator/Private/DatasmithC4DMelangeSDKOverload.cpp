// Copyright Epic Games, Inc. All Rights Reserved.

#ifdef _MELANGE_SDK_

#include "HAL/UnrealMemory.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#include "DatasmithC4DMelangeSDKEnterGuard.h"
#define DONT_INCLUDE_MEMORY_OVERLOADS
#include "default_alien_overloads.h"
#include "DatasmithC4DMelangeSDKLeaveGuard.h"

namespace cineware
{
#pragma pack (push, 8)

	///////////////////////////////////////////////////////////////////////////////////////////////////

	/// Memory allocation functions.
	/// Overload MemAlloc() / MemFree() for custom memory management.

	void* MemAllocNC(Int size)
	{
		// Patch for Melange SDK version 20.004_RBMelange20.0_259890
		// It seems to rely on malloc(0) being non-nullptr, which is not true for FMemory::Malloc.
		// Without this, it will fail to completely read files with geometry that have deleted faces
		return FMemory::Malloc(size > 0 ? size : 1);
	}

	void* MemAlloc(Int size)
	{
		void* mem = MemAllocNC(size);
		if (!mem)
		{
			return nullptr;
		}
		return FMemory::Memset(mem, 0, size);
	}

	void* MemRealloc(void* orimem, Int size)
	{
		return FMemory::Realloc(orimem, size);
	}

	void MemFree(void*& mem)
	{
		if (!mem)
		{
			return;
		}
		FMemory::Free(mem);
		mem = nullptr;
	}

#pragma pack (pop)
}

// overload this function and fill in your own unique data
void GetWriterInfo(cineware::Int32 &id, cineware::String &appname)
{
	// register your own pluginid once for your exporter and enter it here under id
	// this id must be used for your own unique ids
	// 	Bool AddUniqueID(Int32 appid, const Char *const mem, Int32 bytes);
	// 	Bool FindUniqueID(Int32 appid, const Char *&mem, Int32 &bytes) const;
	// 	Bool GetUniqueIDIndex(Int32 idx, Int32 &id, const Char *&mem, Int32 &bytes) const;
	// 	Int32 GetUniqueIDCount() const;

	// The ID was generated with my personal MAXON account
	// (following the instructions here: https://developers.maxon.net/?page_id=3224)
	// it doesn't entail anything so it should be fine
	id = 1050125;

	appname = "Datasmith C4D Importer";
}

#endif //_MELANGE_SDK_
