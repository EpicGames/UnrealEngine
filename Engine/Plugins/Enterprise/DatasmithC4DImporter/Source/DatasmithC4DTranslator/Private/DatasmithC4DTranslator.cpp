// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithC4DTranslator.h"

#if defined(_MELANGE_SDK_)
#include "DatasmithC4DTranslatorModule.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithSceneSource.h"
#include "IDatasmithSceneElements.h"

#include "GenericPlatform/GenericPlatformProcess.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Templates/TypeHash.h"

#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

#define LOCTEXT_NAMESPACE "DatasmithC4DTranslator"

DEFINE_LOG_CATEGORY_STATIC(DatasmithC4DTranslatorLog, Log, All);

void NotifyImprovedImporter();

void FDatasmithC4DTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
	OutCapabilities.bIsEnabled = true;
	OutCapabilities.bParallelLoadStaticMeshSupported = true;

	TArray<FFileFormatInfo>& Formats = OutCapabilities.SupportedFileFormats;
	Formats.Emplace(TEXT("c4d"), TEXT("Cinema 4D file format"));
}

bool FDatasmithC4DTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	NotifyImprovedImporter();

	OutScene->SetHost(TEXT("C4DTranslator"));

	if (!Importer.IsValid())
	{
		FDatasmithC4DImportOptions C4DImportOptions;
		Importer = MakeShared<FDatasmithC4DImporter>(OutScene, C4DImportOptions);
	}

	Importer->OpenFile(GetSource().GetSourceFile());

	return Importer->ProcessScene();
}

void FDatasmithC4DTranslator::UnloadScene()
{
	if (Importer)
	{
		Importer->UnloadScene();
	}
}

bool FDatasmithC4DTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (ensure(Importer.IsValid()))
	{
		TArray<FMeshDescription> MeshDescriptions;
		Importer->GetGeometriesForMeshElementAndRelease(MeshElement, MeshDescriptions);
		if (MeshDescriptions.Num() > 0)
		{
			OutMeshPayload.LodMeshes.Add(MoveTemp(MeshDescriptions[0]));
			return true;
		}
	}

	return false;
}

bool FDatasmithC4DTranslator::LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload)
{
	if (ensure(Importer.IsValid()))
	{
		if (LevelSequenceElement == Importer->GetLevelSequence())
		{
			// #ueent_todo: move data to OutLevelSequencePayload
			return true;
		}
	}

	return false;
}

void FDatasmithC4DTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	Options.Add(GetOrCreateC4DImportOptions());
}

void FDatasmithC4DTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	for (const TObjectPtr<UDatasmithOptionsBase>& OptionPtr : Options)
	{
		if (UDatasmithC4DImportOptions* InImportOptions = Cast<UDatasmithC4DImportOptions>(OptionPtr))
		{
			ImportOptions = InImportOptions;
		}
	}

	if (Importer.IsValid())
	{
		FDatasmithC4DImportOptions C4DImportOptions;
		C4DImportOptions.bImportEmptyMesh = ImportOptions->bImportEmptyMesh;
#if WITH_EDITORONLY_DATA
		C4DImportOptions.bExportToUDatasmith = ImportOptions->bExportToUDatasmith;
#endif
		C4DImportOptions.bAlwaysGenerateNormals = ImportOptions->bAlwaysGenerateNormals;
		C4DImportOptions.ScaleVertices = ImportOptions->ScaleVertices;
		C4DImportOptions.bOptimizeEmptySingleChildActors = ImportOptions->bOptimizeEmptySingleChildActors;

		Importer->SetImportOptions(C4DImportOptions);
	}
}

TObjectPtr<UDatasmithC4DImportOptions>& FDatasmithC4DTranslator::GetOrCreateC4DImportOptions()
{
	if (!ImportOptions)
	{
		ImportOptions = Datasmith::MakeOptionsObjectPtr<UDatasmithC4DImportOptions>();
	}
	return ImportOptions;
}

constexpr auto IMPROVED_IMPORTER_URL = TEXT("https://www.maxon.net/unreal");

void NotifyImprovedImporter()
{
#if WITH_EDITOR
	if (!IsInGameThread() && !IsInSlateThread() && !IsInAsyncLoadingThread())
	{
		return;
	}

	if (IsRunningCommandlet())
	{
		return;
	}

	UE_LOGF(DatasmithC4DTranslatorLog, Warning, "Old DatasmithC4DTranslator loaded");

	const FText Msg = FText::Format(LOCTEXT("DatasmithC4DImporterLoaded",
		"Improved Cineware Import available at {0}"), FText::FromString(IMPROVED_IMPORTER_URL));

	FNotificationInfo Info(Msg);
	Info.ExpireDuration = 8.0f;
	Info.bUseLargeFont = true;
	Info.bUseSuccessFailIcons = true;
	Info.Image = FAppStyle::GetBrush(TEXT("MessageLog.Warning"));
	Info.Hyperlink = FSimpleDelegate::CreateLambda([]() {
		const FString URL = IMPROVED_IMPORTER_URL;
		FPlatformProcess::LaunchURL(*URL, nullptr, nullptr);
		});
	Info.HyperlinkText = LOCTEXT("GoToMaxon", "Go to Maxon...");

	FSlateNotificationManager::Get().AddNotification(Info);
#endif
}

#undef LOCTEXT_NAMESPACE

#endif // _MELANGE_SDK_