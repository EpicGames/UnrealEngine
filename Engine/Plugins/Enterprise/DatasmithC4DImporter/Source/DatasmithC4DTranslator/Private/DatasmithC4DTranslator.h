// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "DatasmithC4DImportOptions.h"
#include "DatasmithTranslator.h"
#include "DatasmithC4DImporter.h"

#include "CoreMinimal.h"

class FDatasmithC4DTranslator : public IDatasmithTranslator
{
public:
	virtual FName GetFName() const override { return "DatasmithC4DTranslator"; };

#if !defined(_MELANGE_SDK_)
	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override { OutCapabilities.bIsEnabled = false; }
#else

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;
	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;
	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;
	virtual bool LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload) override;

	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;

	/** Event for when a C4D document is about to be opened for translation */
	/*DECLARE_EVENT_TwoParams(FDatasmithC4DTranslator, FPreTranslateEvent, cineware::BaseDocument*, const FString&)
	static FPreTranslateEvent& OnPreTranslate() { return PreTranslateEvent; }*/
	
private:
	TObjectPtr<UDatasmithC4DImportOptions>& GetOrCreateC4DImportOptions();

private:
	/** -1: try load dynamic initially; 0: dynamic cannot be loaded; 1: dynamic successful loaded */
	TObjectPtr<UDatasmithC4DImportOptions> ImportOptions;
	TSharedPtr<FDatasmithC4DImporter> Importer;
#endif
};
