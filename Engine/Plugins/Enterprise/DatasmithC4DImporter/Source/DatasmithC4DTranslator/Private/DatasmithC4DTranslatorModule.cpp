// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithC4DTranslatorModule.h"

#ifdef _MELANGE_SDK_

#include "DatasmithC4DTranslator.h"
#include "DatasmithC4DImportOptions.h"

#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"

#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"

#include "Misc/App.h"
#include "Internationalization/Internationalization.h"

#include <vector>

#define LOCTEXT_NAMESPACE "DatasmithC4DTranslatorModule"


class FC4DTranslatorModule : public IDatasmithC4DTranslatorModule
{
public:

	FC4DTranslatorModule()
	{
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
	#if WITH_EDITOR
		const FString EnvVariable = FPlatformMisc::GetEnvironmentVariable(TEXT("DATASMITHC4D_DEBUG"));
		if (!EnvVariable.IsEmpty())
		{
			for (TFieldIterator<FProperty> It(UDatasmithC4DImportOptions::StaticClass()); It; ++It)
			{
				FProperty* Property = *It;
				if (Property && Property->HasMetaData(TEXT("Category")) && Property->GetMetaData(TEXT("Category")) == TEXT("DebugProperty"))
				{
					Property->SetMetaData(TEXT("Category"), TEXT("PrivateSettings"));
				}
			}
		}
	#endif

		// TODO: Load C4DDynamicImporter if available else Import Static

		Datasmith::RegisterTranslator<FDatasmithC4DTranslator>();
	}

	virtual void ShutdownModule() override
	{
		if (FModuleManager::Get().IsModuleLoaded(TEXT("DatasmithTranslator")))
		{
			Datasmith::UnregisterTranslator<FDatasmithC4DTranslator>();
		}
	}
};

#else //_MELANGE_SDK_

// If the _MELANGE_SDK_ is not part of the build, create an empty module
class FC4DTranslatorModule : public IDatasmithC4DTranslatorModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};
#endif //_MELANGE_SDK_

IMPLEMENT_MODULE(FC4DTranslatorModule, DatasmithC4DTranslator);

#undef LOCTEXT_NAMESPACE
