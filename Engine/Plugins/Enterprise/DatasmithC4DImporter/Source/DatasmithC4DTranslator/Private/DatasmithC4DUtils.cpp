// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithC4DUtils.h"

#ifdef _MELANGE_SDK_

#include "CoreMinimal.h"

#include "DatasmithC4DMelangeSDKEnterGuard.h"
#include "c4d_file.h"
#include "DatasmithC4DMelangeSDKLeaveGuard.h"

#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"

int32 MelangeGetInt32(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_LONG)
		{
			return static_cast<int32>(Data.GetInt32());
		}
	}
	return 0;
}

int64 MelangeGetInt64(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_LLONG)
		{
			return static_cast<int64>(Data.GetInt64());
		}
	}
	return 0;
}

bool MelangeGetBool(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_LONG)
		{
			return Data.GetBool();
		}
	}
	return false;
}

double MelangeGetDouble(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data))
		{
			if(Data.GetType() == cineware::DA_REAL)
			{
				return static_cast<double>(Data.GetFloat());
			}
			else if (Data.GetType() == cineware::DA_TIME)
			{
				return static_cast<double>(Data.GetTime().Get());
			}
		}
	}
	return 0.0;
}

FVector MelangeGetVector(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_VECTOR)
		{
			return MelangeVectorToFVector(Data.GetVector());
		}
	}
	return FVector::ZeroVector;
}

FMatrix MelangeGetMatrix(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_MATRIX)
		{
			return MelangeMatrixToFMatrix(Data.GetMatrix());
		}
	}

	FMatrix Result;
	Result.SetIdentity();
	return Result;
}

TArray<uint8> MelangeGetByteArray(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	TArray<uint8> Result;
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_BYTEARRAY)
		{
			const cineware::ByteArray& Arr = Data.GetByteArray();
			Result.SetNumUninitialized(static_cast<int32>(Arr.size));
			FMemory::Memcpy(Result.GetData(), Arr.mem, Arr.size);
		}
	}
	return Result;
}

FString MelangeGetString(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data))
		{
			if(Data.GetType() == cineware::DA_STRING)
			{
				return MelangeStringToFString(Data.GetString());
			}
			else if (Data.GetType() == cineware::DA_FILENAME)
			{
				return MelangeFilenameToPath(Data.GetFilename());
			}
		}
	}
	return FString();
}

cineware::BaseList2D* MelangeGetLink(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(Parameter, Data) && Data.GetType() == cineware::DA_ALIASLINK)
		{
			return Data.GetLink();
		}
	}
	return nullptr;
}

float MelangeGetFloat(cineware::BaseList2D* Object, cineware::Int32 Parameter)
{
	return static_cast<float>(MelangeGetDouble(Object, Parameter));
}

FVector ConvertMelangePosition(const cineware::Vector32& MelangePosition, float WorldUnitScale)
{
	return ConvertMelangePosition(MelangeVectorToFVector(MelangePosition), WorldUnitScale);
}

FVector ConvertMelangePosition(const cineware::Vector64& MelangePosition, float WorldUnitScale)
{
	return ConvertMelangePosition(MelangeVectorToFVector(MelangePosition), WorldUnitScale);
}

FVector ConvertMelangePosition(const FVector& MelangePosition, float WorldUnitScale)
{
	return WorldUnitScale * FVector(MelangePosition.X, -MelangePosition.Z, MelangePosition.Y);
}

FVector ConvertMelangeDirection(const cineware::Vector32& MelangePosition)
{
	return ConvertMelangeDirection(MelangeVectorToFVector(MelangePosition));
}

FVector ConvertMelangeDirection(const cineware::Vector64& MelangePosition)
{
	return ConvertMelangeDirection(MelangeVectorToFVector(MelangePosition));
}

FVector ConvertMelangeDirection(const FVector& MelangePosition)
{
	return FVector(MelangePosition.X, -MelangePosition.Z, MelangePosition.Y);
}

FVector MelangeVectorToFVector(const cineware::Vector32& MelangeVector)
{
	return FVector(MelangeVector.x, MelangeVector.y, MelangeVector.z);
}

FVector MelangeVectorToFVector(const cineware::Vector64& MelangeVector)
{
	return FVector(static_cast<float>(MelangeVector.x),
				   static_cast<float>(MelangeVector.y),
		           static_cast<float>(MelangeVector.z));
}

FVector4 MelangeVector4ToFVector4(const cineware::Vector4d32& MelangeVector)
{
	return FVector4(MelangeVector.x, MelangeVector.y, MelangeVector.z, MelangeVector.w);
}

FVector4 MelangeVector4ToFVector4(const cineware::Vector4d64& MelangeVector)
{
	return FVector4(static_cast<float>(MelangeVector.x),
					static_cast<float>(MelangeVector.y),
					static_cast<float>(MelangeVector.z),
					static_cast<float>(MelangeVector.w));
}

FMatrix MelangeMatrixToFMatrix(const cineware::Matrix& Matrix)
{
	return FMatrix(MelangeVectorToFVector(Matrix.v1),
				   MelangeVectorToFVector(Matrix.v2),
				   MelangeVectorToFVector(Matrix.v3),
				   MelangeVectorToFVector(Matrix.off));
}

FString MelangeStringToFString(const cineware::String& MelangeString)
{
	TUniquePtr<cineware::Char> CStr(MelangeString.GetCStringCopy());
	return CStr.Get();
}

FString MD5FromString(const FString& Value)
{
	FMD5 MD5;
	MD5.Update(reinterpret_cast<const uint8*>(*Value), sizeof(TCHAR)*Value.Len());
	uint8 MD5Hash[16];
	MD5.Final(MD5Hash);
	return BytesToHex(MD5Hash, 16);
}

FString MelangeFilenameToPath(const cineware::Filename& Filename)
{
	return MelangeStringToFString(Filename.GetString());
}

FString SearchForFile(FString Filename, const FString& C4dDocumentFilename)
{
	FPaths::NormalizeFilename(Filename);

	if (FPaths::FileExists(Filename))
	{
		return Filename;
	}

	FString DocumentPath = FPaths::GetPath(C4dDocumentFilename);

	// Try interpreting it as relative to the document path
	if (FPaths::IsRelative(Filename))
	{
		FString AbsolutePath = FPaths::Combine(DocumentPath, Filename);

		if (FPaths::FileExists(AbsolutePath))
		{
			return AbsolutePath;
		}
	}

	// Maybe its a file that has been physically moved to the exported folder, but cineware still
	// has its original filepath
	FString CleanFilename = FPaths::GetCleanFilename(Filename);
	FString LocalPath = FPaths::Combine(DocumentPath, CleanFilename);
	if (FPaths::FileExists(LocalPath))
	{
		return LocalPath;
	}

	// Try searching inside a 'tex' folder first (where cineware emits textures)
	FString PathInTex = FPaths::Combine(DocumentPath, FString("tex"), CleanFilename);
	if (FPaths::FileExists(PathInTex))
	{
		return PathInTex;
	}

	// Last resort: Try a recursive search down from where the document is
	TArray<FString> FoundFiles;
	IFileManager::Get().FindFilesRecursive(FoundFiles, *DocumentPath, *CleanFilename, true, false);
	if (FoundFiles.Num() > 0)
	{
		return FoundFiles[0];
	}

	return "";
}

FString MelangeObjectName(cineware::BaseList2D* Object)
{
	if (!Object)
	{
		return FString(TEXT("Invalid object"));
	}

	return MelangeStringToFString(Object->GetName());
}

FString MelangeObjectTypeName(cineware::BaseList2D* Object)
{
	if (!Object)
	{
		return FString(TEXT("Invalid object"));
	}

	return MelangeStringToFString(cineware::String(cineware::GetObjectTypeName(Object->GetType())));
}

FString GeDataToString(const cineware::GeData& Data)
{
	switch (Data.GetType())
	{
	case cineware::DA_NIL:
		return TEXT("NIL");
	case cineware::DA_VOID:
		return TEXT("VOID");
	case cineware::DA_LONG:
		return FString::Printf(TEXT("%d"), Data.GetInt32());
	case cineware::DA_REAL:
		return FString::Printf(TEXT("%f"), Data.GetFloat());
	case cineware::DA_TIME:
		return FString::Printf(TEXT("%f"), Data.GetTime().Get());
	case cineware::DA_VECTOR:
		return MelangeVectorToFVector(Data.GetVector()).ToString();
	case cineware::DA_MATRIX:
		return MelangeMatrixToFMatrix(Data.GetMatrix()).ToString();
	case cineware::DA_LLONG:
		return FString::Printf(TEXT("%" INT64_FMT), Data.GetInt64());
	case cineware::DA_BYTEARRAY:
	{
		const cineware::ByteArray& Arr = Data.GetByteArray();
		return BytesToHex(static_cast<const uint8*>(Arr.mem), static_cast<int32>(Arr.size));
	}
	case cineware::DA_STRING:
		return MelangeStringToFString(Data.GetString());
	case cineware::DA_FILENAME:
		return MelangeFilenameToPath(Data.GetFilename());
	case cineware::DA_CONTAINER:
		return FString::Printf(TEXT("%d"), Data.GetContainer()->GetId());
	case cineware::DA_ALIASLINK:
	{
		cineware::BaseList2D* Target = Data.GetLink();
		return  FString::Printf(TEXT("%s @0x%llx"), *MelangeObjectName(Target), Target);
	}
	case cineware::DA_MARKER:
		return TEXT("MARKER");
	case cineware::DA_MISSINGPLUG:
		return TEXT("MISSINGPLUG");
	default:
		break;
	}

	return TEXT("UNKNOWN");
}

FString GeTypeToString(int32 GeType)
{
	using GeTypeEntry = TPairInitializer<const int32&, const FString&>;
	const static TMap<int32, FString> GeTypes
	({
		GeTypeEntry(cineware::DA_NIL, TEXT("DA_NIL (no value)")),
		GeTypeEntry(cineware::DA_VOID, TEXT("DA_VOID (void pointer)")),
		GeTypeEntry(cineware::DA_LONG, TEXT("DA_LONG (int32)")),
		GeTypeEntry(cineware::DA_REAL, TEXT("DA_REAL (double)")),
		GeTypeEntry(cineware::DA_TIME, TEXT("DA_TIME (double)")),
		GeTypeEntry(cineware::DA_VECTOR, TEXT("DA_VECTOR (cineware::Vector)")),
		GeTypeEntry(cineware::DA_MATRIX, TEXT("DA_MATRIX (cineware::Matrix)")),
		GeTypeEntry(cineware::DA_LLONG, TEXT("DA_LLONG (int64)")),
		GeTypeEntry(cineware::DA_BYTEARRAY, TEXT("DA_BYTEARRAY (void pointer)")),
		GeTypeEntry(cineware::DA_STRING, TEXT("DA_STRING (cineware::String)")),
		GeTypeEntry(cineware::DA_FILENAME, TEXT("DA_FILENAME (cineware::Filename)")),
		GeTypeEntry(cineware::DA_CONTAINER, TEXT("DA_CONTAINER (cineware::BaseContainer)")),
		GeTypeEntry(cineware::DA_ALIASLINK, TEXT("DA_ALIASLINK (cineware::BaseLink)")),
		GeTypeEntry(cineware::DA_MARKER, TEXT("DA_MARKER (not used)")),
		GeTypeEntry(cineware::DA_MISSINGPLUG, TEXT("DA_MISSINGPLUG (missing datatype plugin)"))
		});

	if (const FString* FoundTypeName = GeTypes.Find(GeType))
	{
		return *FoundTypeName;
	}
	// see c4d_gedata.h
	else if (GeType > 1000000)
	{
		return TEXT("DA_CUSTOMDATATYPE (?)");
	}

	return TEXT("UNKNOWN_TYPE (?)");
}

FString MelangeParameterValueToString(cineware::BaseList2D* Object, cineware::Int32 ParameterID)
{
	if (Object)
	{
		cineware::GeData Data;
		if (Object->GetParameter(ParameterID, Data))
		{
			return GeDataToString(Data);
		}
	}
	return TEXT("");
}

TOptional<FString> GetMelangeBaseList2dID(cineware::BaseList2D* BaseList)
{
	TOptional<FString> MelangeID;
	if (BaseList)
	{
		if (BaseList->GetUniqueIDCount() > 0)
		{
			cineware::Int32 AppId;
			const cineware::Char* IdData;
			cineware::Int Bytes;
			if (BaseList->GetUniqueIDIndex(0, AppId, IdData, Bytes))
			{
				MelangeID.Emplace(BytesToHex(reinterpret_cast<const uint8*>(&AppId), sizeof(AppId)) + "_" + BytesToHex(reinterpret_cast<const uint8*>(IdData), static_cast<int32>(Bytes)));
			}
		}
	}
	
	return MelangeID;
}

#endif
