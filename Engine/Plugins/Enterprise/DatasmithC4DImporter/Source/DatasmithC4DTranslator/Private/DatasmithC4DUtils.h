// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef _MELANGE_SDK_

#include "CoreMinimal.h"

#include "DatasmithC4DMelangeSDKEnterGuard.h"
#include "c4d.h"
#include "DatasmithC4DMelangeSDKLeaveGuard.h"

/**
 * Retrieves the value of a DA_LONG parameter of a cineware object as a cineware::Int32,
 * and static_casts it to an int32. Will return 0 if the parameter is invalid
 */
int32 MelangeGetInt32(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_LLONG parameter of a cineware object
 * and returns it as an int64. Will return 0 if the parameter is invalid
 */
int64 MelangeGetInt64(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_LONG parameter of a cineware object as a cineware::Bool,
 * and returns it as a bool. Will return false if the parameter is invalid
 */
bool MelangeGetBool(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_REAL parameter of a cineware object
 * and returns it as a double. Will return 0.0 if the parameter is invalid
 */
double MelangeGetDouble(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_VECTOR parameter of a cineware object
 * and returns it as an FVector. No coordinate or color conversions are applied.
 * Will return FVector() if the parameter is invalid
 */
FVector MelangeGetVector(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_MATRIX parameter of a cineware object
 * and returns it as an FMatrix. Will return an identity matrix if the parameter is invalid
 */
FMatrix MelangeGetMatrix(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_BYTEARRAY parameter of a cineware object
 * and returns it as an array of bytes. Will return TArray<uint8>() if the parameter is invalid
 */
TArray<uint8> MelangeGetByteArray(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_STRING or DA_FILENAME parameter of a cineware object
 * and returns it as a string. Will return FString() if the parameter is invalid
 */
FString MelangeGetString(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves a the object pointed to by an DA_ALIASLINK parameter of a cineware object
 * and returns it. Will return nullptr if the parameter is invalid
 */
cineware::BaseList2D* MelangeGetLink(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Retrieves the value of a DA_REAL (double) parameter from a cineware object and static_casts it to float.
 * Will return 0.0f if the parameter is invalid
 */
float MelangeGetFloat(cineware::BaseList2D* Object, cineware::Int32 Parameter);

/**
 * Converts a vector representing a position from the cineware coordinate system to the UnrealEditor coordinate system
 */
FVector ConvertMelangePosition(const cineware::Vector32& MelangePosition, float WorldUnitScale = 1.0f);
FVector ConvertMelangePosition(const cineware::Vector64& MelangePosition, float WorldUnitScale = 1.0f);
FVector ConvertMelangePosition(const FVector& MelangePosition, float WorldUnitScale = 1.0f);

/**
 * Converts a vector representing a direction from the cineware coordinate system to the UnrealEditor coordinate system
 */
FVector ConvertMelangeDirection(const cineware::Vector32& MelangePosition);
FVector ConvertMelangeDirection(const cineware::Vector64& MelangePosition);
FVector ConvertMelangeDirection(const FVector& MelangePosition);

/**
 * Converts a cineware::Vector into an FVector
 */
FVector MelangeVectorToFVector(const cineware::Vector32& MelangeVector);
FVector MelangeVectorToFVector(const cineware::Vector64& MelangeVector);

/**
 * Converts a cineware::Vector4d32 into an FVector4
 */
FVector4 MelangeVector4ToFVector4(const cineware::Vector4d32& MelangeVector);

/**
 * Converts a cineware::Vector4d64 into an FVector4
 */
FVector4 MelangeVector4ToFVector4(const cineware::Vector4d64& MelangeVector);

/**
 * Converts a cineware::Matrix into an FMatrix
 */
FMatrix MelangeMatrixToFMatrix(const cineware::Matrix& Matrix);

/**
 * Converts a cineware::String into an FString
 */
FString MelangeStringToFString(const cineware::String& MelangeString);

/**
 * Uses the bytes of 'String' to generate an MD5 hash, and returns the bytes of the hash back as an FString
 */
FString MD5FromString(const FString& String);

/**
 * Converts a cineware::Filename to an FString path
 */
FString MelangeFilenameToPath(const cineware::Filename& Filename);

/**
 * Searches for a file in places where cineware is likely to put them. Will try things like making the filename
 * relative to the C4dDocumentFilename's folder, searching the 'tex' folder, and etc. Use this to search for texture
 * and IES files as the path contained in cineware is likely to be pointing somewhere else if the user didn't export
 * the scene just right.
 *
 * @param Filename				The path to the file we're trying to find. Can be relative or absolute
 * @param C4dDocumentFilename	Path to the main .c4d file being imported e.g. "C:\MyFolder\Scene.c4d"
 * @return						The full, absolute found filepath if it exists, or an empty string
 */
FString SearchForFile(FString Filename, const FString& C4dDocumentFilename);

/**
 * Gets the name of a cineware object as an FString, or returns 'Invalid object' if the argument is nullptr
 */
FString MelangeObjectName(cineware::BaseList2D* Object);

/**
 * Gets the type of a cineware object as an FString (e.g. Ocloner, Onull, Opolygon), or 'Invalid object' if the
 * argument is nullptr
 */
FString MelangeObjectTypeName(cineware::BaseList2D* Object);

/**
 * Gets the data stored within a cineware::GeData object according to its .GetType() and returns that as a string.
 * Will try to convert to Unreal types first (like FVector or FMatrix) and convert those to string instead
 */
FString GeDataToString(const cineware::GeData& Data);

/**
 * Converts the type returned by .GetType() of a cineware::GeData instance from an cineware::Int32 into
 * a human-readable string
 */
FString GeTypeToString(int32 GeType);

/**
 * Gets the corresponding parameter from Object, and returns the value stored in the resulting GeData as a string.
 * Will return an empty string if Object is nullptr, but it can return NIL if the parameter is not found
 */
FString MelangeParameterValueToString(cineware::BaseList2D* Object, cineware::Int32 ParameterID);

/**
 * Returns the full cineware ID for the BaseList2D argument as an FString, which will include AppId
 */
TOptional<FString> GetMelangeBaseList2dID(cineware::BaseList2D* BaseList);

#endif