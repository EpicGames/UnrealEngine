// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#define C4DTRANSLATOR_MODULE_NAME TEXT("DatasmithC4DTranslator")

class IDatasmithC4DTranslatorModule : public IModuleInterface
{

public:
	static inline IDatasmithC4DTranslatorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IDatasmithC4DTranslatorModule >(C4DTRANSLATOR_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(C4DTRANSLATOR_MODULE_NAME);
	}
};
